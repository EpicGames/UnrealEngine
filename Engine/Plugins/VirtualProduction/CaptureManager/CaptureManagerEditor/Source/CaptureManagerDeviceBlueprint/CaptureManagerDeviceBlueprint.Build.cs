// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using UnrealBuildTool.Rules;

public class CaptureManagerDeviceBlueprint : ModuleRules
{
	public CaptureManagerDeviceBlueprint(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CaptureManagerCPSClient",
			"CaptureProtocolStack",
			"CaptureUtils",
		});

		ShortName = "CapManDevBp";
	}
}
