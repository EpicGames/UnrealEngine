// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerDeviceBatchDownloadLibrary.h"

#include "Tasks/Task.h"
#include "Containers/Ticker.h"

#define LOCTEXT_NAMESPACE "CaptureManagerDeviceBatchDownload"

void UCaptureManagerDeviceBatchDownloadLibrary::DownloadDeviceTakesBatch(
	UCaptureManagerDeviceSession* Session,
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	FString DownloadRootDirectory,
	FCaptureManagerBatchDownloadTakeResult OnTakeSuccess,
	FCaptureManagerBatchDownloadTakeFailed OnTakeFailed,
	FCaptureManagerDeviceDownloadProgress OnTakeProgress,
	FCaptureManagerBatchDownloadComplete OnAllComplete)
{
	if (!Session)
	{
		const int32 TakeCount = Takes.Num();
		TArray<FString> TakeNames;
		TakeNames.Reserve(TakeCount);
		for (const FCaptureManagerDeviceTakeInfo& Take : Takes)
		{
			TakeNames.Add(Take.TakeName);
		}
		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[OnTakeFailed, OnAllComplete, TakeNames = MoveTemp(TakeNames), TakeCount]()
			{
				const FText ErrorMessage = LOCTEXT("NullSessionBatch", "Session is null");
				for (int32 Index = 0; Index < TakeCount; ++Index)
				{
					OnTakeFailed.ExecuteIfBound(nullptr, TakeNames[Index], ECaptureManagerDeviceError::InvalidArgument, ErrorMessage, TakeCount - Index - 1);
				}
				OnAllComplete.ExecuteIfBound(nullptr, 0, TakeCount);
			}
		);
		return;
	}

	if (Takes.IsEmpty())
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[OnAllComplete, Session]()
			{
				OnAllComplete.ExecuteIfBound(Session, 0, 0);
			}
		);
		return;
	}

	TStrongObjectPtr<UCaptureManagerDeviceSession> StrongSession(Session);
	TArray<FString> Names;
	Names.Reserve(Takes.Num());
	for (const FCaptureManagerDeviceTakeInfo& Take : Takes)
	{
		Names.Add(Take.TakeName);
	}

	UE::Tasks::Launch(UE_SOURCE_LOCATION,
		[StrongSession = MoveTemp(StrongSession), Names = MoveTemp(Names), DownloadRootDirectory = MoveTemp(DownloadRootDirectory),
		OnTakeSuccess = MoveTemp(OnTakeSuccess), OnTakeFailed = MoveTemp(OnTakeFailed), OnTakeProgress = MoveTemp(OnTakeProgress), OnAllComplete = MoveTemp(OnAllComplete)]() mutable
		{
			StrongSession->ClearDownloadCancelRequest();

			int32 SucceededCount = 0;
			int32 FailedCount = 0;
			const int32 TotalCount = Names.Num();
			UCaptureManagerDeviceSession* SessionPtr = StrongSession.Get();

			auto ReportSkippedTakes = [&OnTakeFailed, &StrongSession, &Names, &FailedCount, TotalCount](int32 StartIndex, ECaptureManagerDeviceError ErrorCode, const FText& ErrorMessage)
				{
					for (int32 SkipIndex = StartIndex; SkipIndex < TotalCount; ++SkipIndex)
					{
						++FailedCount;
						FString SkippedName = Names[SkipIndex];
						int32 SkipRemaining = TotalCount - SkipIndex - 1;
						ExecuteOnGameThread(UE_SOURCE_LOCATION,
							[OnTakeFailed, StrongSession, ErrorCode, SkippedName = MoveTemp(SkippedName), ErrorMessage, SkipRemaining]()
							{
								OnTakeFailed.ExecuteIfBound(StrongSession.Get(), SkippedName, ErrorCode, ErrorMessage, SkipRemaining);
							}
						);
					}
				};

			for (int32 Index = 0; Index < TotalCount; ++Index)
			{
				if (StrongSession->IsDownloadCancelRequested())
				{
					ReportSkippedTakes(Index, ECaptureManagerDeviceError::Canceled, LOCTEXT("BatchCanceled", "Batch canceled"));
					break;
				}

				const FString& TakeName = Names[Index];
				const int32 RemainingCount = TotalCount - Index - 1;

				UCaptureManagerDeviceSession::FOnDownloadProgress ProgressCallback;
				if (OnTakeProgress.IsBound())
				{
					ProgressCallback = [OnTakeProgress, SessionPtr, TakeName](float Progress)
						{
							OnTakeProgress.ExecuteIfBound(SessionPtr, TakeName, Progress);
						};
				}

				FString TakeDirectory;
				FText ErrorMessage;
				ECaptureManagerDeviceError ErrorCode = ECaptureManagerDeviceError::NoError;
				const bool bSuccess = StrongSession->DownloadTake(
					TakeName, DownloadRootDirectory, TakeDirectory, ErrorCode, ErrorMessage, MoveTemp(ProgressCallback));

				if (bSuccess)
				{
					++SucceededCount;
					ExecuteOnGameThread(UE_SOURCE_LOCATION,
						[OnTakeSuccess, StrongSession, TakeName, TakeDirectory = MoveTemp(TakeDirectory), RemainingCount]()
						{
							OnTakeSuccess.ExecuteIfBound(StrongSession.Get(), TakeName, TakeDirectory, RemainingCount);
						}
					);
				}
				else
				{
					++FailedCount;
					ExecuteOnGameThread(UE_SOURCE_LOCATION,
						[OnTakeFailed, StrongSession, TakeName, ErrorCode, ErrorMessage = MoveTemp(ErrorMessage), RemainingCount]()
						{
							OnTakeFailed.ExecuteIfBound(StrongSession.Get(), TakeName, ErrorCode, ErrorMessage, RemainingCount);
						}
					);

					if (ErrorCode == ECaptureManagerDeviceError::Canceled
						|| ErrorCode == ECaptureManagerDeviceError::Disconnected)
					{
						ReportSkippedTakes(Index + 1, ErrorCode, LOCTEXT("BatchAborted", "Batch aborted"));
						break;
					}
				}
			}

			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[OnAllComplete, StrongSession, SucceededCount, FailedCount]()
				{
					OnAllComplete.ExecuteIfBound(StrongSession.Get(), SucceededCount, FailedCount);
				}
			);
		}
	);
}

TArray<FCaptureManagerBatchDownloadResult> UCaptureManagerDeviceBatchDownloadLibrary::DownloadDeviceTakesBatchSync(
	UCaptureManagerDeviceSession* Session,
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	const FString& DownloadRootDirectory)
{
	TArray<FCaptureManagerBatchDownloadResult> Results;

	if (!Session || Takes.IsEmpty())
	{
		return Results;
	}

	Session->ClearDownloadCancelRequest();
	Results.Reserve(Takes.Num());

	auto SkipRemaining = [&Results, &Takes](int32 StartIndex, ECaptureManagerDeviceError ErrorCode, const FText& ErrorMessage)
		{
			for (int32 Index = StartIndex; Index < Takes.Num(); ++Index)
			{
				FCaptureManagerBatchDownloadResult& Skipped = Results.AddDefaulted_GetRef();
				Skipped.TakeName = Takes[Index].TakeName;
				Skipped.ErrorCode = ErrorCode;
				Skipped.ErrorMessage = ErrorMessage;
			}
		};

	for (const FCaptureManagerDeviceTakeInfo& Take : Takes)
	{
		const FString& TakeName = Take.TakeName;

		if (Session->IsDownloadCancelRequested())
		{
			SkipRemaining(Results.Num(), ECaptureManagerDeviceError::Canceled, LOCTEXT("BatchSyncCanceled", "Batch canceled"));
			break;
		}

		FCaptureManagerBatchDownloadResult& Result = Results.AddDefaulted_GetRef();
		Result.TakeName = TakeName;

		Result.bSuccess = Session->DownloadTake(
			TakeName, DownloadRootDirectory, Result.TakeDirectoryPath,
			Result.ErrorCode, Result.ErrorMessage);

		if (Result.ErrorCode == ECaptureManagerDeviceError::Canceled
			|| Result.ErrorCode == ECaptureManagerDeviceError::Disconnected)
		{
			SkipRemaining(Results.Num(), Result.ErrorCode, LOCTEXT("BatchSyncAborted", "Batch aborted"));
			break;
		}
	}

	return Results;
}

void UCaptureManagerDeviceBatchDownloadLibrary::CancelBatchDownload(
	UCaptureManagerDeviceSession* Session)
{
	if (Session)
	{
		Session->CancelAllDownloads();
	}
}

#undef LOCTEXT_NAMESPACE
