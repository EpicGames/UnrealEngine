// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "CaptureManagerDeviceSession.h"
#include "CaptureManagerDeviceBlueprintLibrary.h"

#include "CaptureManagerDeviceBatchDownloadLibrary.generated.h"

DECLARE_DYNAMIC_DELEGATE_FourParams(FCaptureManagerBatchDownloadTakeResult,
	UCaptureManagerDeviceSession*, Session, FString, TakeName, FString, TakeDirectoryPath, int32, RemainingCount);
DECLARE_DYNAMIC_DELEGATE_FiveParams(FCaptureManagerBatchDownloadTakeFailed,
	UCaptureManagerDeviceSession*, Session, FString, TakeName, ECaptureManagerDeviceError, ErrorCode, FText, ErrorMessage, int32, RemainingCount);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FCaptureManagerBatchDownloadComplete,
	UCaptureManagerDeviceSession*, Session, int32, SucceededCount, int32, FailedCount);

USTRUCT(BlueprintType)
struct FCaptureManagerBatchDownloadResult
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Device")
	FString TakeName;

	UPROPERTY(BlueprintReadOnly, Category = "Device")
	bool bSuccess = false;

	UPROPERTY(BlueprintReadOnly, Category = "Device")
	FString TakeDirectoryPath;

	UPROPERTY(BlueprintReadOnly, Category = "Device")
	ECaptureManagerDeviceError ErrorCode = ECaptureManagerDeviceError::NoError;

	UPROPERTY(BlueprintReadOnly, Category = "Device")
	FText ErrorMessage;
};

UCLASS()
class UCaptureManagerDeviceBatchDownloadLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Download multiple takes sequentially from a connected device.
	 *
	 * @param Session                Session handle from ConnectToDevice.
	 * @param Takes                  Takes to download (from GetDeviceTakes, optionally filtered).
	 * @param DownloadRootDirectory  Local directory to download into. Subdirectories are created per take. If a subdirectory already exists it is deleted first.
	 * @param OnTakeSuccess          Called on the game thread when each take completes.
	 * @param OnTakeFailed           Called on the game thread when each take fails.
	 * @param OnTakeProgress         Called on the game thread with download progress (0.0 to 1.0) per take.
	 * @param OnAllComplete          Called on the game thread when all takes have been processed.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device",
		meta = (DisplayName = "Download Device Takes (Batch)",
			Keywords = "iphone ipad livelink batch"))
	static void DownloadDeviceTakesBatch(
		UCaptureManagerDeviceSession* Session,
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		FString DownloadRootDirectory,
		FCaptureManagerBatchDownloadTakeResult OnTakeSuccess,
		FCaptureManagerBatchDownloadTakeFailed OnTakeFailed,
		FCaptureManagerDeviceDownloadProgress OnTakeProgress,
		FCaptureManagerBatchDownloadComplete OnAllComplete);

	/**
	 * Download multiple takes sequentially from a connected device, blocking until all complete.
	 *
	 * @param Session                Session handle from ConnectToDevice.
	 * @param Takes                  Takes to download (from GetDeviceTakes, optionally filtered).
	 * @param DownloadRootDirectory  Local directory to download into. Subdirectories are created per take. If a subdirectory already exists it is deleted first.
	 * @return                       Per-take results with success/failure status and paths.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device|Blocking",
		meta = (DisplayName = "Download Device Takes (Batch)",
			ReturnDisplayName = "Results",
			Keywords = "iphone ipad livelink batch"))
	static TArray<FCaptureManagerBatchDownloadResult> DownloadDeviceTakesBatchSync(
		UCaptureManagerDeviceSession* Session,
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		const FString& DownloadRootDirectory);

	/**
	 * Cancel a batch download. The current take fails with a Canceled error
	 * and remaining takes are skipped.
	 *
	 * @param Session  Session handle from ConnectToDevice.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device",
		meta = (DisplayName = "Cancel Batch Download",
			Keywords = "abort stop batch"))
	static void CancelBatchDownload(UCaptureManagerDeviceSession* Session);
};
