// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerDeviceBlueprintLibrary.h"

#include "Tasks/Task.h"
#include "Containers/Ticker.h"

#define LOCTEXT_NAMESPACE "CaptureManagerDeviceBlueprintLibrary"

static FString ResolveDeviceName(const FString& DeviceName, const FString& IpAddress, int32 Port)
{
	return DeviceName.IsEmpty()
		? FString::Printf(TEXT("%s:%d"), *IpAddress, Port)
		: DeviceName;
}

void UCaptureManagerDeviceBlueprintLibrary::ConnectToDevice(
	FString DeviceName,
	FString IpAddress,
	int32 Port,
	int32 TimeoutSeconds,
	FCaptureManagerDeviceConnected OnSuccess,
	FCaptureManagerDeviceConnectFailed OnFailure)
{
	FString ResolvedName = ResolveDeviceName(DeviceName, IpAddress, Port);

	UCaptureManagerDeviceSession* Session = NewObject<UCaptureManagerDeviceSession>();
	Session->Name = ResolvedName;
	TStrongObjectPtr<UCaptureManagerDeviceSession> StrongSession(Session);

	UE::Tasks::Launch(UE_SOURCE_LOCATION,
		[StrongSession = MoveTemp(StrongSession), ResolvedName = MoveTemp(ResolvedName),
		IpAddress = MoveTemp(IpAddress), Port, TimeoutSeconds, OnSuccess = MoveTemp(OnSuccess), OnFailure = MoveTemp(OnFailure)]() mutable
		{
			ECaptureManagerDeviceError ErrorCode = ECaptureManagerDeviceError::NoError;
			FText ErrorMessage;
			const bool bConnected = StrongSession->Connect(
				IpAddress, Port, TimeoutSeconds, ErrorCode, ErrorMessage);

			TStrongObjectPtr<UCaptureManagerDeviceSession> ResultSession;
			if (bConnected)
			{
				ResultSession = MoveTemp(StrongSession);
			}

			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[ResultSession = MoveTemp(ResultSession), ResolvedName = MoveTemp(ResolvedName),
				ErrorCode, ErrorMessage = MoveTemp(ErrorMessage), OnSuccess, OnFailure]()
				{
					if (ResultSession.IsValid())
					{
						OnSuccess.ExecuteIfBound(ResultSession.Get());
					}
					else
					{
						OnFailure.ExecuteIfBound(ResolvedName, ErrorCode, ErrorMessage);
					}
				}
			);
		}
	);
}

void UCaptureManagerDeviceBlueprintLibrary::GetDeviceTakes(
	UCaptureManagerDeviceSession* Session,
	FCaptureManagerDeviceGetTakesResult OnSuccess,
	FCaptureManagerDeviceGetTakesFailed OnFailure)
{
	if (!Session)
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[OnFailure]()
			{
				OnFailure.ExecuteIfBound(nullptr, ECaptureManagerDeviceError::InvalidArgument, LOCTEXT("NullSessionGetTakes", "Session is null"));
			}
		);
		return;
	}

	TStrongObjectPtr<UCaptureManagerDeviceSession> StrongSession(Session);

	UE::Tasks::Launch(UE_SOURCE_LOCATION,
		[StrongSession = MoveTemp(StrongSession), OnSuccess = MoveTemp(OnSuccess), OnFailure = MoveTemp(OnFailure)]() mutable
		{
			ECaptureManagerDeviceError ErrorCode = ECaptureManagerDeviceError::NoError;
			FText ErrorMessage;
			TArray<FCaptureManagerDeviceTakeInfo> Takes = GetDeviceTakesSync(StrongSession.Get(), ErrorCode, ErrorMessage);

			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[StrongSession, ErrorCode, Takes = MoveTemp(Takes), ErrorMessage = MoveTemp(ErrorMessage), OnSuccess, OnFailure]()
				{
					if (ErrorCode == ECaptureManagerDeviceError::NoError)
					{
						OnSuccess.ExecuteIfBound(StrongSession.Get(), Takes);
					}
					else
					{
						OnFailure.ExecuteIfBound(StrongSession.Get(), ErrorCode, ErrorMessage);
					}
				}
			);
		}
	);
}

void UCaptureManagerDeviceBlueprintLibrary::DownloadDeviceTake(
	UCaptureManagerDeviceSession* Session,
	FString TakeName,
	FString DownloadRootDirectory,
	FCaptureManagerDeviceDownloadResult OnSuccess,
	FCaptureManagerDeviceDownloadFailed OnFailure,
	FCaptureManagerDeviceDownloadProgress OnProgress)
{
	if (!Session)
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[OnFailure, TakeName]()
			{
				OnFailure.ExecuteIfBound(nullptr, TakeName, ECaptureManagerDeviceError::InvalidArgument, LOCTEXT("NullSessionDownload", "Session is null"));
			}
		);
		return;
	}

	TStrongObjectPtr<UCaptureManagerDeviceSession> StrongSession(Session);

	UE::Tasks::Launch(UE_SOURCE_LOCATION,
		[StrongSession = MoveTemp(StrongSession), TakeName = MoveTemp(TakeName), DownloadRootDirectory = MoveTemp(DownloadRootDirectory), OnSuccess = MoveTemp(OnSuccess), OnFailure = MoveTemp(OnFailure), OnProgress = MoveTemp(OnProgress)]() mutable
		{
			UCaptureManagerDeviceSession::FOnDownloadProgress ProgressCallback;
			if (OnProgress.IsBound())
			{
				UCaptureManagerDeviceSession* SessionPtr = StrongSession.Get();
				ProgressCallback = [OnProgress, SessionPtr, TakeName](float Progress)
					{
						OnProgress.ExecuteIfBound(SessionPtr, TakeName, Progress);
					};
			}

			FString TakeDirectory;
			FText ErrorMessage;
			ECaptureManagerDeviceError ErrorCode = ECaptureManagerDeviceError::NoError;
			const bool bSuccess = StrongSession->DownloadTake(TakeName, DownloadRootDirectory, TakeDirectory, ErrorCode, ErrorMessage, MoveTemp(ProgressCallback));

			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[StrongSession, bSuccess, ErrorCode, TakeName = MoveTemp(TakeName), TakeDirectory = MoveTemp(TakeDirectory), ErrorMessage = MoveTemp(ErrorMessage), OnSuccess, OnFailure]()
				{
					if (bSuccess)
					{
						OnSuccess.ExecuteIfBound(StrongSession.Get(), TakeName, TakeDirectory);
					}
					else
					{
						OnFailure.ExecuteIfBound(StrongSession.Get(), TakeName, ErrorCode, ErrorMessage);
					}
				}
			);
		}
	);
}

bool UCaptureManagerDeviceBlueprintLibrary::CancelDeviceDownload(
	UCaptureManagerDeviceSession* Session,
	FString TakeName,
	FText& ErrorMessage)
{
	if (!Session)
	{
		ErrorMessage = LOCTEXT("NullSessionCancel", "Session is null");
		return false;
	}

	return Session->CancelDownload(TakeName, ErrorMessage);
}

void UCaptureManagerDeviceBlueprintLibrary::DisconnectDevice(
	UCaptureManagerDeviceSession* Session,
	FCaptureManagerDeviceDisconnected OnComplete)
{
	if (!Session)
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[OnComplete]()
			{
				OnComplete.ExecuteIfBound();
			}
		);
		return;
	}

	TStrongObjectPtr<UCaptureManagerDeviceSession> StrongSession(Session);

	UE::Tasks::Launch(UE_SOURCE_LOCATION,
		[StrongSession = MoveTemp(StrongSession), OnComplete = MoveTemp(OnComplete)]()
		{
			StrongSession->Disconnect();

			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[OnComplete]()
				{
					OnComplete.ExecuteIfBound();
				}
			);
		}
	);
}

void UCaptureManagerDeviceBlueprintLibrary::DisconnectDeviceSync(UCaptureManagerDeviceSession* Session)
{
	if (Session)
	{
		Session->Disconnect();
	}
}

UCaptureManagerDeviceSession* UCaptureManagerDeviceBlueprintLibrary::ConnectToDeviceSync(
	const FString& DeviceName,
	const FString& IpAddress,
	int32 Port,
	int32 TimeoutSeconds,
	ECaptureManagerDeviceError& ErrorCode,
	FText& ErrorMessage)
{
	ErrorCode = ECaptureManagerDeviceError::NoError;

	UCaptureManagerDeviceSession* Session = NewObject<UCaptureManagerDeviceSession>();
	Session->Name = ResolveDeviceName(DeviceName, IpAddress, Port);

	if (!Session->Connect(IpAddress, Port, TimeoutSeconds, ErrorCode, ErrorMessage))
	{
		return nullptr;
	}

	return Session;
}

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceBlueprintLibrary::GetDeviceTakesSync(
	UCaptureManagerDeviceSession* Session,
	ECaptureManagerDeviceError& ErrorCode,
	FText& ErrorMessage)
{
	TArray<FCaptureManagerDeviceTakeInfo> Takes;
	ErrorCode = ECaptureManagerDeviceError::NoError;

	if (!Session)
	{
		ErrorCode = ECaptureManagerDeviceError::InvalidArgument;
		ErrorMessage = LOCTEXT("NullSessionGetTakesSync", "Session is null");
		return Takes;
	}

	Session->FetchTakes(Takes, ErrorCode, ErrorMessage);
	return Takes;
}

FString UCaptureManagerDeviceBlueprintLibrary::DownloadDeviceTakeSync(
	UCaptureManagerDeviceSession* Session,
	const FString& TakeName,
	const FString& DownloadRootDirectory,
	ECaptureManagerDeviceError& ErrorCode,
	FText& ErrorMessage)
{
	ErrorCode = ECaptureManagerDeviceError::NoError;

	if (!Session)
	{
		ErrorCode = ECaptureManagerDeviceError::InvalidArgument;
		ErrorMessage = LOCTEXT("NullSessionDownloadSync", "Session is null");
		return FString();
	}

	FString TakeDirectory;
	if (!Session->DownloadTake(TakeName, DownloadRootDirectory, TakeDirectory, ErrorCode, ErrorMessage))
	{
		return FString();
	}

	return TakeDirectory;
}

#undef LOCTEXT_NAMESPACE
