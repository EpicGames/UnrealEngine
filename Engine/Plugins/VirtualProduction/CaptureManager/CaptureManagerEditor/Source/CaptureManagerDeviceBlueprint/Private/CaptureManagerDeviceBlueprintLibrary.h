// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "CaptureManagerDeviceSession.h"

#include "CaptureManagerDeviceBlueprintLibrary.generated.h"

DECLARE_DYNAMIC_DELEGATE_OneParam(FCaptureManagerDeviceConnected, UCaptureManagerDeviceSession*, Session);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FCaptureManagerDeviceConnectFailed, FString, DeviceName, ECaptureManagerDeviceError, ErrorCode, FText, ErrorMessage);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FCaptureManagerDeviceGetTakesResult, UCaptureManagerDeviceSession*, Session, const TArray<FCaptureManagerDeviceTakeInfo>&, Takes);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FCaptureManagerDeviceGetTakesFailed, UCaptureManagerDeviceSession*, Session, ECaptureManagerDeviceError, ErrorCode, FText, ErrorMessage);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FCaptureManagerDeviceDownloadResult, UCaptureManagerDeviceSession*, Session, FString, TakeName, FString, TakeDirectoryPath);
DECLARE_DYNAMIC_DELEGATE_FourParams(FCaptureManagerDeviceDownloadFailed, UCaptureManagerDeviceSession*, Session, FString, TakeName, ECaptureManagerDeviceError, ErrorCode, FText, ErrorMessage);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FCaptureManagerDeviceDownloadProgress, UCaptureManagerDeviceSession*, Session, FString, TakeName, float, Progress);
DECLARE_DYNAMIC_DELEGATE(FCaptureManagerDeviceDisconnected);

UCLASS()
class UCaptureManagerDeviceBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Connect to a device (e.g. LiveLink Face on iPhone/iPad) by IP address.
	 * Returns a session handle via OnSuccess, used by all subsequent device operations.
	 *
	 * Note: The device uses a second port for file transfers, reported during
	 * connection. If downloads fail while the connection succeeds, check that
	 * this port is also accessible through any firewalls.
	 *
	 * @param DeviceName      Optional display name for the device. Stored on the session for use in logging and error messages. Defaults to "IP:Port" if empty.
	 * @param IpAddress       IP address of the device.
	 * @param Port            Default is 14785.
	 * @param TimeoutSeconds  Maximum time to wait for connection.
	 * @param OnSuccess       Called on the game thread when connected.
	 * @param OnFailure       Called on the game thread if the connection fails or times out.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device",
		meta = (DisplayName = "Connect to Device",
			Keywords = "iphone ipad livelink",
			Port = "14785", TimeoutSeconds = "30"))
	static void ConnectToDevice(
		FString DeviceName,
		FString IpAddress,
		int32 Port,
		int32 TimeoutSeconds,
		FCaptureManagerDeviceConnected OnSuccess,
		FCaptureManagerDeviceConnectFailed OnFailure
	);

	/**
	 * Retrieve the list of takes available on a connected device.
	 *
	 * @param Session    Session handle from ConnectToDevice.
	 * @param OnSuccess  Called on the game thread when the fetch succeeds.
	 * @param OnFailure  Called on the game thread if the fetch fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device",
		meta = (DisplayName = "Get Device Takes",
			Keywords = "iphone ipad livelink fetch"))
	static void GetDeviceTakes(
		UCaptureManagerDeviceSession* Session,
		FCaptureManagerDeviceGetTakesResult OnSuccess,
		FCaptureManagerDeviceGetTakesFailed OnFailure
	);

	/**
	 * Download a take from a connected device to a local directory.
	 * The downloaded directory can be passed directly to Ingest Live Link Face.
	 *
	 * @param Session                Session handle from ConnectToDevice.
	 * @param TakeName               Name of the take to download (from GetDeviceTakes).
	 * @param DownloadRootDirectory  Local directory to download into. A subdirectory named after the take is created automatically. If the subdirectory already exists it is deleted first.
	 * @param OnSuccess              Called on the game thread when the download completes.
	 * @param OnFailure              Called on the game thread if the download fails.
	 * @param OnProgress             Called on the game thread with download progress (0.0 to 1.0).
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device",
		meta = (DisplayName = "Download Device Take", Keywords = "iphone ipad livelink"))
	static void DownloadDeviceTake(
		UCaptureManagerDeviceSession* Session,
		FString TakeName,
		FString DownloadRootDirectory,
		FCaptureManagerDeviceDownloadResult OnSuccess,
		FCaptureManagerDeviceDownloadFailed OnFailure,
		FCaptureManagerDeviceDownloadProgress OnProgress
	);

	/**
	 * Cancel an in-flight download by take name.
	 * The download's OnFailure callback will fire with a cancellation error.
	 *
	 * @param Session       Session handle from ConnectToDevice.
	 * @param TakeName      Name of the take whose download to cancel.
	 * @param ErrorMessage  Populated on failure with a description of why cancellation failed.
	 * @return              True if a download for this take was found and cancellation was initiated. False if no active download exists for this take.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device",
		meta = (DisplayName = "Cancel Device Download", Keywords = "abort stop"))
	static bool CancelDeviceDownload(
		UCaptureManagerDeviceSession* Session,
		FString TakeName,
		FText& ErrorMessage);

	/**
	 * Disconnect from a device and release the session.
	 * Cancels any in-flight downloads. Safe to call with a null session.
	 *
	 * @param Session     Session handle to disconnect.
	 * @param OnComplete  Called on the game thread when the session is fully torn down.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device",
		meta = (DisplayName = "Disconnect Device", Keywords = "close"))
	static void DisconnectDevice(
		UCaptureManagerDeviceSession* Session,
		FCaptureManagerDeviceDisconnected OnComplete
	);

	/**
	 * Disconnect from a device, blocking until teardown is complete.
	 * Cancels any in-flight downloads. Safe to call with a null session.
	 *
	 * @param Session  Session handle to disconnect.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device|Blocking",
		meta = (DisplayName = "Disconnect Device", Keywords = "close"))
	static void DisconnectDeviceSync(UCaptureManagerDeviceSession* Session);

	/**
	 * Connect to a device and block until connected or timed out.
	 * Intended for Python scripts and other contexts where blocking is acceptable.
	 *
	 * @param DeviceName      Optional display name for the device. Defaults to "IP:Port" if empty.
	 * @param IpAddress       IP address of the device.
	 * @param Port            Default is 14785.
	 * @param TimeoutSeconds  Maximum time to wait for connection.
	 * @param ErrorCode       Error code on failure.
	 * @param ErrorMessage    Error description on failure.
	 * @return                Session handle on success, nullptr on failure.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device|Blocking",
		meta = (DisplayName = "Connect to Device",
			ReturnDisplayName = "Session",
			Keywords = "iphone ipad livelink",
			Port = "14785",
			TimeoutSeconds = "30"))
	static UCaptureManagerDeviceSession* ConnectToDeviceSync(
		const FString& DeviceName,
		const FString& IpAddress,
		int32 Port,
		int32 TimeoutSeconds,
		ECaptureManagerDeviceError& ErrorCode,
		FText& ErrorMessage
	);

	/**
	 * Retrieve the list of takes available on a connected device, blocking until complete.
	 *
	 * @param Session       Session handle from ConnectToDevice.
	 * @param ErrorCode     Error code on failure.
	 * @param ErrorMessage  Error description on failure.
	 * @return              Array of take info. Empty on failure or if no takes exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device|Blocking",
		meta = (DisplayName = "Get Device Takes",
			ReturnDisplayName = "Takes",
			Keywords = "iphone ipad livelink fetch"))
	static TArray<FCaptureManagerDeviceTakeInfo> GetDeviceTakesSync(
		UCaptureManagerDeviceSession* Session,
		ECaptureManagerDeviceError& ErrorCode,
		FText& ErrorMessage
	);

	/**
	 * Download a take from a connected device, blocking until complete.
	 * The returned directory can be passed directly to Ingest Live Link Face.
	 *
	 * @param Session                Session handle from ConnectToDevice.
	 * @param TakeName               Name of the take to download (from GetDeviceTakes).
	 * @param DownloadRootDirectory  Local directory to download into. A subdirectory named after the take is created automatically. If the subdirectory already exists it is deleted first.
	 * @param ErrorCode              Error code on failure.
	 * @param ErrorMessage           Error description on failure.
	 * @return                       Full path to the downloaded take directory. Empty on failure.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Device|Blocking",
		meta = (DisplayName = "Download Device Take",
			ReturnDisplayName = "TakeDirectoryPath",
			Keywords = "iphone ipad livelink"))
	static FString DownloadDeviceTakeSync(
		UCaptureManagerDeviceSession* Session,
		const FString& TakeName,
		const FString& DownloadRootDirectory,
		ECaptureManagerDeviceError& ErrorCode,
		FText& ErrorMessage
	);
};
