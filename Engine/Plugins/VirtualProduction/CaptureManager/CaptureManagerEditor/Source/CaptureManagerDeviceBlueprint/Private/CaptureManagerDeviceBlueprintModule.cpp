// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerDeviceBlueprintModule.h"

#include "CaptureManagerDeviceSession.h"
#include "Modules/ModuleManager.h"

void FCaptureManagerDeviceBlueprintModule::StartupModule()
{
}

void FCaptureManagerDeviceBlueprintModule::ShutdownModule()
{
	TArray<TStrongObjectPtr<UCaptureManagerDeviceSession>> Sessions;
	{
		FScopeLock Lock(&SessionsMutex);
		Sessions = MoveTemp(ActiveSessions);
	}
	for (TStrongObjectPtr<UCaptureManagerDeviceSession>& Session : Sessions)
	{
		Session->Disconnect();
	}
}

void FCaptureManagerDeviceBlueprintModule::RegisterSession(UCaptureManagerDeviceSession* Session)
{
	if (Session)
	{
		FScopeLock Lock(&SessionsMutex);
		ActiveSessions.Emplace(Session);
	}
}

void FCaptureManagerDeviceBlueprintModule::UnregisterSession(UCaptureManagerDeviceSession* Session)
{
	FScopeLock Lock(&SessionsMutex);
	ActiveSessions.RemoveAll([Session](const TStrongObjectPtr<UCaptureManagerDeviceSession>& Entry)
		{
			return Entry.Get() == Session;
		});
}

IMPLEMENT_MODULE(FCaptureManagerDeviceBlueprintModule, CaptureManagerDeviceBlueprint)
