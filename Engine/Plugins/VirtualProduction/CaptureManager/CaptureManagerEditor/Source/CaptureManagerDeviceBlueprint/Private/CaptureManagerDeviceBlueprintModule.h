// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Modules/ModuleInterface.h"
#include "CaptureManagerDeviceSession.h"
#include "UObject/StrongObjectPtr.h"

class FCaptureManagerDeviceBlueprintModule : public IModuleInterface
{
public:
	void StartupModule() override;
	void ShutdownModule() override;

	void RegisterSession(UCaptureManagerDeviceSession* Session);
	void UnregisterSession(UCaptureManagerDeviceSession* Session);

private:
	FCriticalSection SessionsMutex;
	TArray<TStrongObjectPtr<UCaptureManagerDeviceSession>> ActiveSessions;
};
