// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerDeviceSession.h"

#include <atomic>

#include "CaptureManagerDeviceBlueprintModule.h"
#include "Containers/Ticker.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Modules/ModuleManager.h"
#include "CPSDevice.h"
#include "CPSFileStream.h"
#include "Control/Messages/ControlResponse.h"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureManagerDevice, Log, All);

static TAutoConsoleVariable<float> CVarDisconnectTimeoutSeconds(
	TEXT("CaptureManager.Device.DisconnectTimeoutSeconds"),
	10.0f,
	TEXT("Seconds to wait for in-flight operations to drain during disconnect."));

static TAutoConsoleVariable<float> CVarDownloadTimeoutSeconds(
	TEXT("CaptureManager.Device.DownloadTimeoutSeconds"),
	1800.0f,
	TEXT("Maximum seconds to wait for a single take download before giving up."));

#define LOCTEXT_NAMESPACE "CaptureManagerDeviceBlueprint"

namespace UE::CaptureManager
{

struct FOperationGate
{
	FOperationGate()
	{
		OperationsComplete->Trigger();
	}

	class FGuard
	{
	public:
		FGuard() = default;

		~FGuard()
		{
			if (Gate)
			{
				Gate->Release();
			}
		}

		FGuard(FGuard&& Other) : Gate(Other.Gate) { Other.Gate = nullptr; }

		FGuard& operator=(FGuard&& Other)
		{
			if (this != &Other)
			{
				if (Gate)
				{
					Gate->Release();
				}
				Gate = Other.Gate;
				Other.Gate = nullptr;
			}
			return *this;
		}

		FGuard(const FGuard&) = delete;
		FGuard& operator=(const FGuard&) = delete;

		explicit operator bool() const { return Gate != nullptr; }

	private:
		friend struct FOperationGate;
		explicit FGuard(FOperationGate* InGate) : Gate(InGate) {}
		FOperationGate* Gate = nullptr;
	};

	FGuard Acquire()
	{
		FScopeLock Lock(&Mutex);
		if (++ActiveCount == 1)
		{
			OperationsComplete->Reset();
		}
		return FGuard(this);
	}

	bool WaitForAll(FTimespan Timeout)
	{
		return OperationsComplete->Wait(Timeout);
	}

private:
	void Release()
	{
		FScopeLock Lock(&Mutex);
		if (--ActiveCount == 0)
		{
			OperationsComplete->Trigger();
		}
	}

	FCriticalSection Mutex;
	int32 ActiveCount = 0;
	FEventRef OperationsComplete{ EEventMode::ManualReset };
};

struct FDownloadRegistry
{
	FTakeId Register(const FString& TakeName)
	{
		FScopeLock Lock(&Mutex);
		if (ActiveDownloads.Contains(TakeName))
		{
			return 0;
		}
		FTakeId Id = NextId++;
		ActiveDownloads.Add(TakeName, Id);
		return Id;
	}

	void Unregister(const FString& TakeName)
	{
		FScopeLock Lock(&Mutex);
		ActiveDownloads.Remove(TakeName);
	}

	TOptional<FTakeId> Find(const FString& TakeName)
	{
		FScopeLock Lock(&Mutex);
		const FTakeId* FoundId = ActiveDownloads.Find(TakeName);
		if (!FoundId)
		{
			return {};
		}
		return *FoundId;
	}

private:
	FCriticalSection Mutex;
	TMap<FString, FTakeId> ActiveDownloads;
	int32 NextId = 1;
};

struct FScopedOperation
{
	TSharedPtr<UCaptureManagerDeviceSession::FImpl, ESPMode::ThreadSafe> Impl;
	UE::CaptureManager::FOperationGate::FGuard Guard;
	explicit operator bool() const { return Impl.IsValid(); }
	UCaptureManagerDeviceSession::FImpl* operator->() const { return Impl.Get(); }
};

} // namespace UE::CaptureManager

// In-flight operations hold a shared_ptr to FImpl so they can finish gracefully
// after Disconnect() nulls the session's Impl pointer. Disconnect() signals
// cancellation and waits with a timeout, but the shared_ptr keeps FImpl alive
// until the operation's stack unwinds.
struct UCaptureManagerDeviceSession::FImpl
{
	TSharedPtr<UE::CaptureManager::FCPSDevice> Device;
	UE::CaptureManager::FOperationGate OperationGate;
	UE::CaptureManager::FDownloadRegistry Downloads;
	std::atomic<bool> bDownloadCancelRequested{ false };
};

TSharedPtr<UCaptureManagerDeviceSession::FImpl, ESPMode::ThreadSafe>
UCaptureManagerDeviceSession::AcquireImpl() const
{
	FScopeLock Lock(&ImplMutex);
	return Impl;
}

UE::CaptureManager::FScopedOperation UCaptureManagerDeviceSession::AcquireImplForOperation() const
{
	using namespace UE::CaptureManager;

	FScopeLock Lock(&ImplMutex);
	if (!Impl || !Impl->Device || !Impl->Device->IsConnected())
	{
		return {};
	}
	return FScopedOperation{ Impl, Impl->OperationGate.Acquire() };
}

UCaptureManagerDeviceSession::~UCaptureManagerDeviceSession()
{
	Disconnect();
}

bool UCaptureManagerDeviceSession::Connect(const FString& IpAddress, int32 Port, int32 TimeoutSeconds, ECaptureManagerDeviceError& OutErrorCode, FText& OutErrorMessage)
{
	using namespace UE::CaptureManager;

	Disconnect();

	{
		FScopeLock Lock(&ImplMutex);
		bConnectCanceled = false;
	}

	OutErrorCode = ECaptureManagerDeviceError::NoError;

	if (IpAddress.IsEmpty())
	{
		OutErrorCode = ECaptureManagerDeviceError::InvalidArgument;
		OutErrorMessage = LOCTEXT("EmptyIpAddress", "IP address must not be empty");
		return false;
	}

	if (Port <= 0 || Port > 65535)
	{
		OutErrorCode = ECaptureManagerDeviceError::InvalidArgument;
		OutErrorMessage = LOCTEXT("InvalidPort", "Port must be between 1 and 65535");
		return false;
	}

	TSharedPtr<FImpl, ESPMode::ThreadSafe> NewImpl = MakeShared<FImpl, ESPMode::ThreadSafe>();
	NewImpl->Device = FCPSDevice::MakeCPSDevice(IpAddress, static_cast<uint16>(Port));

	if (!NewImpl->Device)
	{
		OutErrorCode = ECaptureManagerDeviceError::Unknown;
		OutErrorMessage = LOCTEXT("FailedToCreateDevice", "Failed to create device connection");
		return false;
	}

	TSharedRef<FEventRef> ConnectedEvent = MakeShared<FEventRef>();

	NewImpl->Device->SubscribeToEvent(
		FConnectionStateChangedEvent::Name,
		FCaptureEventHandler(
			[ConnectedEvent](TSharedPtr<const FCaptureEvent> InEvent)
			{
				const FConnectionStateChangedEvent& ConnectionEvent = static_cast<const FConnectionStateChangedEvent&>(*InEvent);
				if (ConnectionEvent.ConnectionState == FConnectionStateChangedEvent::EState::Connected)
				{
					(*ConnectedEvent)->Trigger();
				}
			},
			EDelegateExecutionThread::InternalThread
		)
	);

	NewImpl->Device->InitiateConnect();

	const bool bConnected = (*ConnectedEvent)->Wait(FTimespan::FromSeconds(TimeoutSeconds));

	NewImpl->Device->UnsubscribeAll();

	if (!bConnected)
	{
		OutErrorCode = ECaptureManagerDeviceError::ConnectionTimeout;
		OutErrorMessage = LOCTEXT("ConnectionTimedOut", "Timed out waiting for device to connect");
		NewImpl->Device->Stop();
		return false;
	}

	{
		FScopeLock Lock(&ImplMutex);
		if (bConnectCanceled)
		{
			NewImpl->Device->Stop();
			OutErrorCode = ECaptureManagerDeviceError::Canceled;
			OutErrorMessage = LOCTEXT("ConnectCanceled", "Connect was canceled by a concurrent disconnect");
			return false;
		}
		Impl = MoveTemp(NewImpl);
	}
	FModuleManager::GetModuleChecked<FCaptureManagerDeviceBlueprintModule>("CaptureManagerDeviceBlueprint").RegisterSession(this);
	return true;
}

void UCaptureManagerDeviceSession::Disconnect()
{
	if (FCaptureManagerDeviceBlueprintModule* Module = FModuleManager::GetModulePtr<FCaptureManagerDeviceBlueprintModule>("CaptureManagerDeviceBlueprint"))
	{
		Module->UnregisterSession(this);
	}
	TSharedPtr<FImpl, ESPMode::ThreadSafe> LocalImpl;
	{
		FScopeLock Lock(&ImplMutex);
		bConnectCanceled = true;
		LocalImpl = MoveTemp(Impl);
	}
	if (LocalImpl && LocalImpl->Device)
	{
		LocalImpl->Device->CancelAllExports();
		if (!LocalImpl->OperationGate.WaitForAll(FTimespan::FromSeconds(CVarDisconnectTimeoutSeconds.GetValueOnAnyThread())))
		{
			UE_LOG(LogCaptureManagerDevice, Error, TEXT("Timed out waiting for operations to complete during disconnect"));
		}
		LocalImpl->Device->Stop();
	}
}

bool UCaptureManagerDeviceSession::FetchTakes(TArray<FCaptureManagerDeviceTakeInfo>& OutTakes, ECaptureManagerDeviceError& OutErrorCode, FText& OutErrorMessage)
{
	using namespace UE::CaptureManager;

	OutErrorCode = ECaptureManagerDeviceError::NoError;

	FScopedOperation Op = AcquireImplForOperation();
	if (!Op)
	{
		OutErrorCode = ECaptureManagerDeviceError::Disconnected;
		OutErrorMessage = LOCTEXT("NotConnected", "Device is not connected");
		return false;
	}

	TProtocolResult<TArray<FGetTakeMetadataResponse::FTakeObject>> Result = Op->Device->FetchTakeList();

	if (Result.HasError())
	{
		OutErrorCode = ECaptureManagerDeviceError::ProtocolError;
		OutErrorMessage = FText::FromString(Result.GetError().GetMessage());
		return false;
	}

	OutTakes.Empty();

	for (const FGetTakeMetadataResponse::FTakeObject& Take : Result.GetValue())
	{
		FCaptureManagerDeviceTakeInfo Info;
		Info.TakeName = Take.Name;
		Info.Slate = Take.Slate;
		Info.TakeNumber = Take.TakeNumber;
		if (!FDateTime::ParseIso8601(*Take.DateTime, Info.DateTime))
		{
			UE_LOG(LogCaptureManagerDevice, Warning, TEXT("Failed to parse DateTime '%s' for take '%s'"), *Take.DateTime, *Take.Name);
		}
		int64 TotalSize = 0;
		for (const FGetTakeMetadataResponse::FFileObject& File : Take.Files)
		{
			TotalSize += static_cast<int64>(File.Length);
		}
		Info.TotalSizeBytes = TotalSize;

		OutTakes.Add(MoveTemp(Info));
	}

	return true;
}

bool UCaptureManagerDeviceSession::DownloadTake(const FString& TakeName, const FString& DownloadRootDirectory, FString& OutTakeDirectory, ECaptureManagerDeviceError& OutErrorCode, FText& OutErrorMessage, FOnDownloadProgress OnProgress)
{
	using namespace UE::CaptureManager;

	OutErrorCode = ECaptureManagerDeviceError::NoError;

	FScopedOperation Op = AcquireImplForOperation();
	if (!Op)
	{
		OutErrorCode = ECaptureManagerDeviceError::Disconnected;
		OutErrorMessage = LOCTEXT("NotConnectedDownload", "Device is not connected");
		return false;
	}

	TProtocolResult<FGetTakeMetadataResponse::FTakeObject> FetchResult = Op->Device->FetchTake(TakeName);
	if (FetchResult.HasError())
	{
		OutErrorCode = ECaptureManagerDeviceError::TakeNotFound;
		OutErrorMessage = FText::Format(
			LOCTEXT("TakeNotFoundOnDeviceFormat", "Take '{0}' not found on device: {1}"),
			FText::FromString(TakeName),
			FText::FromString(FetchResult.GetError().GetMessage())
		);
		return false;
	}

	FGetTakeMetadataResponse::FTakeObject Take = FetchResult.StealValue();

	FTakeId TakeId = Op->Downloads.Register(TakeName);
	if (!TakeId)
	{
		OutErrorCode = ECaptureManagerDeviceError::InvalidArgument;
		OutErrorMessage = FText::Format(
			LOCTEXT("DuplicateDownloadFormat", "Take '{0}' is already being downloaded"),
			FText::FromString(TakeName)
		);
		return false;
	}

	ON_SCOPE_EXIT
	{
		Op->Downloads.Unregister(TakeName);
	};

	FString TakeDirectory = FPaths::Combine(DownloadRootDirectory, TakeName);
	FPaths::CollapseRelativeDirectories(TakeDirectory);
	if (!FPaths::IsUnderDirectory(TakeDirectory, DownloadRootDirectory))
	{
		OutErrorCode = ECaptureManagerDeviceError::InvalidArgument;
		OutErrorMessage = FText::Format(
			LOCTEXT("TakeNameInvalidPathFormat", "Take name '{0}' contains path components that resolve outside the download directory"),
			FText::FromString(TakeName)
		);
		return false;
	}

	if (IFileManager::Get().DirectoryExists(*TakeDirectory))
	{
		UE_LOG(LogCaptureManagerDevice, Log, TEXT("Replacing existing take directory: %s"), *TakeDirectory);
		if (!IFileManager::Get().DeleteDirectory(*TakeDirectory, false, true))
		{
			OutErrorCode = ECaptureManagerDeviceError::DownloadFailed;
			OutErrorMessage = FText::Format(
				LOCTEXT("DeleteDirectoryFailedFormat", "Failed to remove existing take directory '{0}'"),
				FText::FromString(TakeDirectory)
			);
			return false;
		}
	}

	Op->Device->AddTakeMetadata(TakeId, Take);

	uint64 TotalSize = 0;
	for (const FGetTakeMetadataResponse::FFileObject& File : Take.Files)
	{
		TotalSize += File.Length;
	}

	TSharedRef<FEventRef> ExportCompleteEvent = MakeShared<FEventRef>();
	TSharedRef<TProtocolResult<void>> ExportResult = MakeShared<TProtocolResult<void>>(TInPlaceType<void>{});

	TUniquePtr<FCPSFileStream> Stream = MakeUnique<FCPSFileStream>(DownloadRootDirectory, TotalSize);

	if (OnProgress && TotalSize > 0)
	{
		TSharedPtr<std::atomic<float>> LastReportedProgress = MakeShared<std::atomic<float>>(0.0f);
		Stream->SetProgressHandler(
			FCPSFileStream::FReportProgress::CreateLambda(
				[OnProgress, LastReportedProgress](float InProgress)
				{
					if (InProgress - *LastReportedProgress >= 0.01f)
					{
						*LastReportedProgress = InProgress;
						FTSTicker::GetCoreTicker().AddTicker(
							FTickerDelegate::CreateLambda(
								[OnProgress, InProgress](float)
								{
									OnProgress(InProgress);
									return false;
								}
							)
						);
					}
				}
			)
		);
	}

	Stream->SetExportFinished(
		FCPSFileStream::FExportFinished::CreateLambda(
			[ExportCompleteEvent, ExportResult, OnProgress](TProtocolResult<void> InResult)
			{
				if (OnProgress && !InResult.HasError())
				{
					FTSTicker::GetCoreTicker().AddTicker(
						FTickerDelegate::CreateLambda(
							[OnProgress](float)
							{
								OnProgress(1.0f);
								return false;
							}
						)
					);
				}
				*ExportResult = MoveTemp(InResult);
				(*ExportCompleteEvent)->Trigger();
			}
		)
	);

	Op->Device->StartExport(TakeId, MoveTemp(Stream));

	bool bDisconnected = false;
	bool bTimedOut = false;
	const double TimeoutSeconds = CVarDownloadTimeoutSeconds.GetValueOnAnyThread();
	const double StartTime = FPlatformTime::Seconds();
	while (!(*ExportCompleteEvent)->Wait(FTimespan::FromSeconds(1.0)))
	{
		if (!Op->Device->IsConnected())
		{
			bDisconnected = true;
			break;
		}
		if (FPlatformTime::Seconds() - StartTime >= TimeoutSeconds)
		{
			bTimedOut = true;
			break;
		}
	}

	if (bDisconnected || bTimedOut)
	{
		Op->Device->CancelExport(TakeId);
	}

	Op->Device->RemoveTakeMetadata(TakeId);

	if (bDisconnected)
	{
		OutErrorCode = ECaptureManagerDeviceError::Disconnected;
		OutErrorMessage = LOCTEXT("DisconnectedDuringDownload", "Device disconnected during download");
		return false;
	}

	if (bTimedOut)
	{
		OutErrorCode = ECaptureManagerDeviceError::DownloadFailed;
		OutErrorMessage = LOCTEXT("DownloadTimedOut", "Download timed out");
		return false;
	}

	if (ExportResult->HasError())
	{
		OutErrorCode = ExportResult->GetError().GetCode() == FExportClient::AbortedError
			? ECaptureManagerDeviceError::Canceled
			: ECaptureManagerDeviceError::DownloadFailed;
		OutErrorMessage = FText::FromString(ExportResult->GetError().GetMessage());
		return false;
	}

	OutTakeDirectory = TakeDirectory;
	return true;
}

bool UCaptureManagerDeviceSession::CancelDownload(const FString& TakeName, FText& OutErrorMessage)
{
	using namespace UE::CaptureManager;

	TSharedPtr<FImpl, ESPMode::ThreadSafe> LocalImpl = AcquireImpl();
	if (!LocalImpl || !LocalImpl->Device)
	{
		OutErrorMessage = LOCTEXT("NotConnectedCancel", "Device is not connected");
		return false;
	}

	TOptional<FTakeId> TakeId = LocalImpl->Downloads.Find(TakeName);
	if (!TakeId.IsSet())
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("NoActiveDownloadFormat", "No active download for take '{0}'"),
			FText::FromString(TakeName)
		);
		return false;
	}

	LocalImpl->Device->CancelExport(TakeId.GetValue());
	return true;
}

void UCaptureManagerDeviceSession::CancelAllDownloads()
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> LocalImpl = AcquireImpl();
	if (!LocalImpl)
	{
		return;
	}
	LocalImpl->bDownloadCancelRequested.store(true);
	// Precondition: CPS serializes exports - at most one is in-flight per session.
	if (LocalImpl->Device)
	{
		LocalImpl->Device->CancelAllExports();
	}
}

bool UCaptureManagerDeviceSession::IsDownloadCancelRequested() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> LocalImpl = AcquireImpl();
	return LocalImpl && LocalImpl->bDownloadCancelRequested.load();
}

void UCaptureManagerDeviceSession::ClearDownloadCancelRequest()
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> LocalImpl = AcquireImpl();
	if (LocalImpl)
	{
		LocalImpl->bDownloadCancelRequested.store(false);
	}
}

bool UCaptureManagerDeviceSession::IsConnected() const
{
	TSharedPtr<FImpl, ESPMode::ThreadSafe> LocalImpl = AcquireImpl();
	return LocalImpl && LocalImpl->Device && LocalImpl->Device->IsConnected();
}

#undef LOCTEXT_NAMESPACE
