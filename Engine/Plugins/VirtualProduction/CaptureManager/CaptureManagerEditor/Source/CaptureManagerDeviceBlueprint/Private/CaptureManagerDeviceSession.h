// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Misc/DateTime.h"
#include "UObject/Object.h"

#include "CaptureManagerDeviceSession.generated.h"

namespace UE::CaptureManager
{
struct FScopedOperation;
}

UENUM(BlueprintType)
enum class ECaptureManagerDeviceError : uint8
{
	NoError            UMETA(DisplayName = "No Error"),
	Unknown            UMETA(DisplayName = "Unknown"),
	InvalidArgument    UMETA(DisplayName = "Invalid Argument"),
	ConnectionTimeout  UMETA(DisplayName = "Connection Timeout"),
	Disconnected       UMETA(DisplayName = "Disconnected"),
	TakeNotFound       UMETA(DisplayName = "Take Not Found"),
	DownloadFailed     UMETA(DisplayName = "Download Failed"),
	Canceled           UMETA(DisplayName = "Canceled"),
	ProtocolError      UMETA(DisplayName = "Protocol Error"),
};

/** Metadata for a single take on a connected device. */
USTRUCT(BlueprintType)
struct FCaptureManagerDeviceTakeInfo
{
	GENERATED_BODY()

	/** Unique name identifying this take on the device. */
	UPROPERTY(BlueprintReadOnly, Category = "Device")
	FString TakeName;

	/** Slate Name. */
	UPROPERTY(BlueprintReadOnly, Category = "Device")
	FString Slate;

	/** Take number within the slate. */
	UPROPERTY(BlueprintReadOnly, Category = "Device")
	int32 TakeNumber = 0;

	/** When this take was recorded. */
	UPROPERTY(BlueprintReadOnly, Category = "Device")
	FDateTime DateTime;

	/** Total size of all files in this take, in bytes. */
	UPROPERTY(BlueprintReadOnly, Category = "Device")
	int64 TotalSizeBytes = 0;
};

/** Represents a connection to a device. Obtained from UCaptureManagerDeviceBlueprintLibrary and passed to all device operations. */
UCLASS(BlueprintType)
class UCaptureManagerDeviceSession : public UObject
{
	// Thread safety: public methods are safe to call from any thread. The async Blueprint wrappers
	// dispatch to background threads because these calls block. All mutable session state lives in
	// FImpl behind ImplMutex and per-subsystem locks.
	GENERATED_BODY()

public:

	~UCaptureManagerDeviceSession();

	/** Display name for this device. Defaults to "IP:Port" on connect. Set to a friendly name if desired. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Device")
	FString Name;

	bool Connect(const FString& IpAddress, int32 Port, int32 TimeoutSeconds, ECaptureManagerDeviceError& OutErrorCode, FText& OutErrorMessage);
	void Disconnect();

	bool FetchTakes(TArray<FCaptureManagerDeviceTakeInfo>& OutTakes, ECaptureManagerDeviceError& OutErrorCode, FText& OutErrorMessage);

	using FOnDownloadProgress = TFunction<void(float Progress)>;
	bool DownloadTake(const FString& TakeName, const FString& DownloadRootDirectory, FString& OutTakeDirectory, ECaptureManagerDeviceError& OutErrorCode, FText& OutErrorMessage, FOnDownloadProgress OnProgress = nullptr);

	bool CancelDownload(const FString& TakeName, FText& OutErrorMessage);

	void CancelAllDownloads();
	bool IsDownloadCancelRequested() const;
	void ClearDownloadCancelRequest();

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Device")
	bool IsConnected() const;

private:

	struct FImpl;
	friend struct UE::CaptureManager::FScopedOperation;
	// Returns the current FImpl, or null if disconnected. Does not prevent Disconnect() from proceeding.
	TSharedPtr<FImpl, ESPMode::ThreadSafe> AcquireImpl() const;
	// Returns a scoped handle that keeps an operation in-flight so Disconnect() waits for it to finish.
	// The operation is automatically released when the handle is destroyed.
	UE::CaptureManager::FScopedOperation AcquireImplForOperation() const;

	mutable FCriticalSection ImplMutex;
	TSharedPtr<FImpl, ESPMode::ThreadSafe> Impl;
	bool bConnectCanceled = false;
};
