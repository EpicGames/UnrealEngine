// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerDeviceTakeFiltersLibrary.h"

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceTakeFiltersLibrary::FilterTakesBySlate(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	FString SlatePattern,
	bool bExclude)
{
	if (SlatePattern.IsEmpty())
	{
		return Takes;
	}

	TArray<FCaptureManagerDeviceTakeInfo> Result;
	for (const FCaptureManagerDeviceTakeInfo& Take : Takes)
	{
		const bool bMatches = Take.Slate.MatchesWildcard(SlatePattern, ESearchCase::IgnoreCase);
		if (bMatches != bExclude)
		{
			Result.Add(Take);
		}
	}
	return Result;
}

static TArray<FCaptureManagerDeviceTakeInfo> SortAndTruncate(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	int32 Count,
	TFunctionRef<bool(const FCaptureManagerDeviceTakeInfo&, const FCaptureManagerDeviceTakeInfo&)> Comparator)
{
	TArray<FCaptureManagerDeviceTakeInfo> Sorted = Takes;
	Sorted.Sort(Comparator);
	if (Count > 0 && Count < Sorted.Num())
	{
		Sorted.SetNum(Count);
	}
	return Sorted;
}

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceTakeFiltersLibrary::GetLatestTakes(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	int32 Count)
{
	return SortAndTruncate(Takes, Count, [](const FCaptureManagerDeviceTakeInfo& A, const FCaptureManagerDeviceTakeInfo& B)
		{
			return A.DateTime > B.DateTime;
		});
}

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceTakeFiltersLibrary::GetOldestTakes(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	int32 Count)
{
	return SortAndTruncate(Takes, Count, [](const FCaptureManagerDeviceTakeInfo& A, const FCaptureManagerDeviceTakeInfo& B)
		{
			return A.DateTime < B.DateTime;
		});
}

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceTakeFiltersLibrary::GetLargestTakes(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	int32 Count)
{
	return SortAndTruncate(Takes, Count, [](const FCaptureManagerDeviceTakeInfo& A, const FCaptureManagerDeviceTakeInfo& B)
		{
			return A.TotalSizeBytes > B.TotalSizeBytes;
		});
}

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceTakeFiltersLibrary::GetSmallestTakes(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	int32 Count)
{
	return SortAndTruncate(Takes, Count, [](const FCaptureManagerDeviceTakeInfo& A, const FCaptureManagerDeviceTakeInfo& B)
		{
			return A.TotalSizeBytes < B.TotalSizeBytes;
		});
}

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceTakeFiltersLibrary::GetLatestSlate(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes)
{
	if (Takes.IsEmpty())
	{
		return {};
	}

	const FCaptureManagerDeviceTakeInfo* Latest = &Takes[0];
	for (const FCaptureManagerDeviceTakeInfo& Take : Takes)
	{
		if (Take.DateTime > Latest->DateTime)
		{
			Latest = &Take;
		}
	}

	const FString& SlateName = Latest->Slate;

	TArray<FCaptureManagerDeviceTakeInfo> Result;
	for (const FCaptureManagerDeviceTakeInfo& Take : Takes)
	{
		if (Take.Slate == SlateName)
		{
			Result.Add(Take);
		}
	}
	return Result;
}

TArray<FCaptureManagerDeviceTakeInfo> UCaptureManagerDeviceTakeFiltersLibrary::FilterTakesByDateRange(
	const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
	FDateTime AfterDateTime,
	FDateTime BeforeDateTime)
{
	const bool bHasAfter = AfterDateTime != FDateTime();
	const bool bHasBefore = BeforeDateTime != FDateTime();

	if (!bHasAfter && !bHasBefore)
	{
		return Takes;
	}

	TArray<FCaptureManagerDeviceTakeInfo> Result;
	for (const FCaptureManagerDeviceTakeInfo& Take : Takes)
	{
		if (bHasAfter && Take.DateTime < AfterDateTime)
		{
			continue;
		}
		if (bHasBefore && Take.DateTime > BeforeDateTime)
		{
			continue;
		}
		Result.Add(Take);
	}
	return Result;
}
