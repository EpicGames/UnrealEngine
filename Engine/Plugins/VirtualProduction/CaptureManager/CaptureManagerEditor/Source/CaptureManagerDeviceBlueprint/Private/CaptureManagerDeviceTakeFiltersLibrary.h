// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "CaptureManagerDeviceSession.h"

#include "CaptureManagerDeviceTakeFiltersLibrary.generated.h"

UCLASS()
class UCaptureManagerDeviceTakeFiltersLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Filter takes by slate name. Supports * wildcard. Case-insensitive.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CaptureManager|Device|Filters",
		meta = (DisplayName = "Filter Takes (Slate)",
			ReturnDisplayName = "FilteredTakes",
			Keywords = "wildcard exclude"))
	static TArray<FCaptureManagerDeviceTakeInfo> FilterTakesBySlate(
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		FString SlatePattern,
		bool bExclude = false);

	/**
	 * Filter takes by date range. Either bound can be omitted - leave the
	 * pin unconnected in Blueprint, or pass FDateTime() in C++.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CaptureManager|Device|Filters",
		meta = (DisplayName = "Filter Takes (Date Range)", 
			ReturnDisplayName = "FilteredTakes",
			Keywords = "time before after range"))
	static TArray<FCaptureManagerDeviceTakeInfo> FilterTakesByDateRange(
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		FDateTime AfterDateTime,
		FDateTime BeforeDateTime);

	/**
	 * Return the N most recent takes, sorted newest first.
	 * Pass 1 to get the single latest take. Pass 0 or negative to return all takes sorted.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CaptureManager|Device|Filters",
		meta = (DisplayName = "Get Latest Takes",
			ReturnDisplayName = "Takes",
			Keywords = "recent last newest",
			Count = "1"))
	static TArray<FCaptureManagerDeviceTakeInfo> GetLatestTakes(
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		int32 Count);

	/**
	 * Return the N oldest takes, sorted oldest first.
	 * Pass 1 to get the single oldest take. Pass 0 or negative to return all takes sorted.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CaptureManager|Device|Filters",
		meta = (DisplayName = "Get Oldest Takes",
			ReturnDisplayName = "Takes",
			Keywords = "oldest first earliest",
			Count = "1"))
	static TArray<FCaptureManagerDeviceTakeInfo> GetOldestTakes(
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		int32 Count);

	/**
	 * Return the N largest takes by total file size, sorted largest first.
	 * Pass 1 to get the single largest take. Pass 0 or negative to return all takes sorted.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CaptureManager|Device|Filters",
		meta = (DisplayName = "Get Largest Takes",
			ReturnDisplayName = "Takes",
			Keywords = "size bytes biggest",
			Count = "1"))
	static TArray<FCaptureManagerDeviceTakeInfo> GetLargestTakes(
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		int32 Count);

	/**
	 * Return the N smallest takes by total file size, sorted smallest first.
	 * Pass 1 to get the single smallest take. Pass 0 or negative to return all takes sorted.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CaptureManager|Device|Filters",
		meta = (DisplayName = "Get Smallest Takes",
			ReturnDisplayName = "Takes",
			Keywords = "size bytes smallest least",
			Count = "1"))
	static TArray<FCaptureManagerDeviceTakeInfo> GetSmallestTakes(
		const TArray<FCaptureManagerDeviceTakeInfo>& Takes,
		int32 Count);

	/**
	 * Return all takes from the most recent slate. Finds the newest take,
	 * then returns every take with the same slate name.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CaptureManager|Device|Filters",
		meta = (DisplayName = "Get Latest Slate",
			ReturnDisplayName = "Takes",
			Keywords = "recent last newest current"))
	static TArray<FCaptureManagerDeviceTakeInfo> GetLatestSlate(const TArray<FCaptureManagerDeviceTakeInfo>& Takes);
};
