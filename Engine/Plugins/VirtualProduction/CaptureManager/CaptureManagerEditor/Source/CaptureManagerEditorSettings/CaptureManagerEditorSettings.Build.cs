// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using UnrealBuildTool.Rules;

public class CaptureManagerEditorSettings : ModuleRules
{
	public CaptureManagerEditorSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"CaptureUtils"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"DeveloperSettings",
			"CaptureDataConverter",
			"CaptureManagerStyle",
			"ContentBrowser",
			"ContentBrowserData",
			"LiveLinkHubCaptureMessaging",
			"LiveLinkHubExportServer",
			"LiveLinkHubMessaging",
			"LiveLinkInterface",
			"NamingTokens",
			"PropertyEditor",
			"Slate",
			"SlateCore",
			"Settings",
			"ToolMenus",
			"UnrealEd"
		});

		ShortName = "CapManEdSet";
	}
}
