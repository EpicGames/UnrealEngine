// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerEditorSettingsModule.h"

#include "ISettingsModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Settings/CaptureManagerEditorSettings.h"
#include "Settings/CaptureManagerEditorSettingsCustomization.h"

#define LOCTEXT_NAMESPACE "CaptureManagerEditorSettings"

void FCaptureManagerEditorSettingsModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FCaptureManagerEditorSettingsModule::PostEngineInit);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FCaptureManagerEditorSettingsModule::EnginePreExit);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(UCaptureManagerEditorSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FCaptureManagerEditorSettingsCustomization::MakeInstance));
}

void FCaptureManagerEditorSettingsModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}

void FCaptureManagerEditorSettingsModule::PostEngineInit()
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	check(Settings);
	Settings->Initialize();
}

void FCaptureManagerEditorSettingsModule::EnginePreExit()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Capture Manager");
	}
}

IMPLEMENT_MODULE(FCaptureManagerEditorSettingsModule, CaptureManagerEditorSettings)

#undef LOCTEXT_NAMESPACE