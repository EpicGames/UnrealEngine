// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FSlateStyleSet;

class FCaptureManagerEditorSettingsModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	void PostEngineInit();
	void EnginePreExit();

	TSharedPtr<FSlateStyleSet> StyleSet;
};
