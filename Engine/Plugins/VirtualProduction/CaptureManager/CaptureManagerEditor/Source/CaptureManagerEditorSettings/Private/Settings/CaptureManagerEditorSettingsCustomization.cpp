// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerEditorSettingsCustomization.h"

#include "Settings/CaptureManagerEditorTemplateTokens.h"
#include "Settings/CaptureManagerEditorSettings.h"
#include "CaptureManagerEncoderNamingTokens.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "NamingTokensEngineSubsystem.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CaptureManagerEditorSettingsCustomization"

TSharedRef<IDetailCustomization> FCaptureManagerEditorSettingsCustomization::MakeInstance()
{
	return MakeShared<FCaptureManagerEditorSettingsCustomization>();
}

void FCaptureManagerEditorSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// Update current value when opening the settings page. Not safe to do under the settings object since there isn't
	// an explicit callback when the settings page is opened, and using a method like PostInitProperties fires on the CDO
	// early in the startup process.

	auto BuildSlate = [this](const TSharedRef<IPropertyHandle> InHandle, IDetailPropertyRow* InPropertyRow, const FNamingTokenFilterArgs& InTokenArgs) {
		InPropertyRow->CustomWidget()
			.WholeRowContent()
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(0.f, 4.f)
					.AutoHeight()
					[
						SNew(STextBlock)
							.Text(InHandle->GetPropertyDisplayName())
							.Font(IDetailLayoutBuilder::GetDetailFontBold())
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SEditableText)
							.IsReadOnly(true)
							.Text_Raw(this, &FCaptureManagerEditorSettingsCustomization::GetDisplayTokenText, InTokenArgs)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
		};

	FNamingTokenFilterArgs DefaultArgs({ {}, false });

	// Import tokens
	const TSharedRef<IPropertyHandle> ImportTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, ImportTokens));
	IDetailPropertyRow* ImportTokensRow = InDetailBuilder.EditDefaultProperty(ImportTokensHandle);
	check(ImportTokensRow);

	FNamingTokenFilterArgs ImportArgs = DefaultArgs;
	if (const TObjectPtr<const UCaptureManagerIngestNamingTokens> Tokens = GetDefault<UCaptureManagerEditorSettings>()->GetGeneralNamingTokens())
	{
		ImportArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
	}
	BuildSlate(ImportTokensHandle, ImportTokensRow, ImportArgs);

	// Video tokens
	const TSharedRef<IPropertyHandle> VideoTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, VideoTokens));
	IDetailPropertyRow* VideoTokensRow = InDetailBuilder.EditDefaultProperty(VideoTokensHandle);
	check(VideoTokensRow);

	FNamingTokenFilterArgs VideoArgs = DefaultArgs;
	if (const TObjectPtr<const UCaptureManagerVideoNamingTokens> VideoTokens = GetDefault<UCaptureManagerEditorSettings>()->GetVideoNamingTokens())
	{
		VideoArgs.AdditionalNamespacesToInclude.Add(VideoTokens->GetNamespace());
	}
	BuildSlate(VideoTokensHandle, VideoTokensRow, VideoArgs);

	// Audio tokens
	const TSharedRef<IPropertyHandle> AudioTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, AudioTokens));
	IDetailPropertyRow* AudioTokensRow = InDetailBuilder.EditDefaultProperty(AudioTokensHandle);
	check(AudioTokensRow);

	FNamingTokenFilterArgs AudioArgs = DefaultArgs;
	if (const TObjectPtr<const UCaptureManagerAudioNamingTokens> AudioTokens = GetDefault<UCaptureManagerEditorSettings>()->GetAudioNamingTokens())
	{
		AudioArgs.AdditionalNamespacesToInclude.Add(AudioTokens->GetNamespace());
	}
	BuildSlate(AudioTokensHandle, AudioTokensRow, AudioArgs);

	// Calibration tokens
	const TSharedRef<IPropertyHandle> CalibTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, CalibrationTokens));
	IDetailPropertyRow* CalibTokensRow = InDetailBuilder.EditDefaultProperty(CalibTokensHandle);
	check(CalibTokensRow);

	FNamingTokenFilterArgs CalibArgs = DefaultArgs;
	if (const TObjectPtr<const UCaptureManagerCalibrationNamingTokens> CalibTokens = GetDefault<UCaptureManagerEditorSettings>()->GetCalibrationNamingTokens())
	{
		CalibArgs.AdditionalNamespacesToInclude.Add(CalibTokens->GetNamespace());
	}
	BuildSlate(CalibTokensHandle, CalibTokensRow, CalibArgs);

	// Lens File tokens
	const TSharedRef<IPropertyHandle> LensFileTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, LensFileTokens));
	IDetailPropertyRow* LensFileTokensRow = InDetailBuilder.EditDefaultProperty(LensFileTokensHandle);
	check(LensFileTokensRow);

	FNamingTokenFilterArgs LensFileArgs = DefaultArgs;
	if (const TObjectPtr<const UCaptureManagerLensFileNamingTokens> LensFileTokens = GetDefault<UCaptureManagerEditorSettings>()->GetLensFileNamingTokens())
	{
		LensFileArgs.AdditionalNamespacesToInclude.Add(LensFileTokens->GetNamespace());
	}
	BuildSlate(LensFileTokensHandle, LensFileTokensRow, LensFileArgs);

	// Video command tokens
	const TSharedRef<IPropertyHandle> VideoCommandTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, VideoCommandTokens));
	IDetailPropertyRow* VideoCommandTokensRow = InDetailBuilder.EditDefaultProperty(VideoCommandTokensHandle);
	check(VideoCommandTokensRow);

	FNamingTokenFilterArgs VideoCommandArgs = DefaultArgs;
	if (const TObjectPtr<const UCaptureManagerVideoEncoderTokens> VideoTokens = GetDefault<UCaptureManagerEditorSettings>()->GetVideoEncoderNamingTokens())
	{
		VideoCommandArgs.AdditionalNamespacesToInclude.Add(VideoTokens->GetNamespace());
	}
	BuildSlate(VideoCommandTokensHandle, VideoCommandTokensRow, VideoCommandArgs);

	// Audio command tokens
	const TSharedRef<IPropertyHandle> AudioCommandTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, AudioCommandTokens));
	IDetailPropertyRow* AudioCommandTokensRow = InDetailBuilder.EditDefaultProperty(AudioCommandTokensHandle);
	check(AudioCommandTokensRow);

	FNamingTokenFilterArgs AudioCommandArgs = DefaultArgs;
	if (const TObjectPtr<const UCaptureManagerAudioEncoderTokens> AudioTokens = GetDefault<UCaptureManagerEditorSettings>()->GetAudioEncoderNamingTokens())
	{
		AudioCommandArgs.AdditionalNamespacesToInclude.Add(AudioTokens->GetNamespace());
	}
	BuildSlate(AudioCommandTokensHandle, AudioCommandTokensRow, AudioCommandArgs);

	// Global tokens
	const TSharedRef<IPropertyHandle> GlobalTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerEditorSettings, GlobalTokens));
	IDetailPropertyRow* GlobalTokensRow = InDetailBuilder.EditDefaultProperty(GlobalTokensHandle);
	check(GlobalTokensRow);

	BuildSlate(GlobalTokensHandle, GlobalTokensRow, FNamingTokenFilterArgs());
}

FText FCaptureManagerEditorSettingsCustomization::GetDisplayTokenText(FNamingTokenFilterArgs InArgs) const
{
	FString FormattedTokensString = TEXT("None");
	if (GEngine)
	{
		FormattedTokensString = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->GetFormattedTokensStringForDisplay(MoveTemp(InArgs));
	}
	return FText::FromString(FormattedTokensString);
}

#undef LOCTEXT_NAMESPACE