// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/CaptureManagerEditorTemplateTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CaptureManagerEditorTemplateTokens)

#define LOCTEXT_NAMESPACE "CaptureManagerEditorNamingTokens"


UCaptureManagerIngestNamingTokens::UCaptureManagerIngestNamingTokens()
{
	Namespace = TEXT("cmi");

	IngestGeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::IdKey), { FString(UE::CaptureManager::GeneralTokens::IdKey), LOCTEXT("DeviceId", "Unique Id")});
	IngestGeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::DeviceKey), { FString(UE::CaptureManager::GeneralTokens::DeviceKey), LOCTEXT("DeviceName", "Device User Name")});
	IngestGeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::SlateKey), { FString(UE::CaptureManager::GeneralTokens::SlateKey), LOCTEXT("DeviceSlate", "Slate")});
	IngestGeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::TakeKey), { FString(UE::CaptureManager::GeneralTokens::TakeKey), LOCTEXT("DeviceTake", "Take")});
}

UE::CaptureManager::FIngestToken UCaptureManagerIngestNamingTokens::GetToken(const FString& InKey) const
{
	check(IngestGeneralTokens.Contains(InKey));
	return IngestGeneralTokens[InKey];
}

void UCaptureManagerIngestNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	Super::OnCreateDefaultTokens(OutTokens);

	for (const TPair<FString, UE::CaptureManager::FIngestToken>& Token : IngestGeneralTokens)
	{
		OutTokens.Emplace(
			Token.Value.Name,
			Token.Value.Description,
			FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda(
				[Name = Token.Value.Name] {
					return FText::FromString(Name);
				}
			)
		);
	}
	
}

void UCaptureManagerIngestNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerIngestNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}


UCaptureManagerVideoNamingTokens::UCaptureManagerVideoNamingTokens()
{
	Namespace = TEXT("cmv");

	IngestVideoTokens.Emplace(FString(UE::CaptureManager::VideoTokens::NameKey), { FString(UE::CaptureManager::VideoTokens::NameKey), LOCTEXT("VideoName", "Name (used to identify a track of recorded data)") });
	IngestVideoTokens.Emplace(FString(UE::CaptureManager::VideoTokens::FrameRateKey), { FString(UE::CaptureManager::VideoTokens::FrameRateKey), LOCTEXT("VideoFrameRate", "Frame Rate") });
}

UE::CaptureManager::FIngestToken UCaptureManagerVideoNamingTokens::GetToken(const FString& InKey) const
{
	check(IngestVideoTokens.Contains(InKey));
	return IngestVideoTokens[InKey];
}

void UCaptureManagerVideoNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	Super::OnCreateDefaultTokens(OutTokens);

	for (const TPair<FString, UE::CaptureManager::FIngestToken>& Token : IngestVideoTokens)
	{
		OutTokens.Emplace(
			Token.Value.Name,
			Token.Value.Description,
			FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda(
				[Name = Token.Value.Name] {
					return FText::FromString(Name);
				}
			)
		);
	}

}

void UCaptureManagerVideoNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerVideoNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}


UCaptureManagerAudioNamingTokens::UCaptureManagerAudioNamingTokens()
{
	Namespace = TEXT("cma");

	IngestAudioTokens.Emplace(FString(UE::CaptureManager::AudioTokens::NameKey), { FString(UE::CaptureManager::AudioTokens::NameKey), LOCTEXT("AudioName", "Name (used to identify a track of recorded data)") });
}

UE::CaptureManager::FIngestToken UCaptureManagerAudioNamingTokens::GetToken(const FString& InKey) const
{
	check(IngestAudioTokens.Contains(InKey));
	return IngestAudioTokens[InKey];
}

void UCaptureManagerAudioNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	Super::OnCreateDefaultTokens(OutTokens);

	for (const TPair<FString, UE::CaptureManager::FIngestToken>& Token : IngestAudioTokens)
	{
		OutTokens.Emplace(
			Token.Value.Name,
			Token.Value.Description,
			FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda(
				[Name = Token.Value.Name] {
					return FText::FromString(Name);
				}
			)
		);
	}

}

void UCaptureManagerAudioNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerAudioNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}


UCaptureManagerCalibrationNamingTokens::UCaptureManagerCalibrationNamingTokens()
{
	Namespace = TEXT("cmc");

	IngestCalibTokens.Emplace(FString(UE::CaptureManager::CalibTokens::NameKey), { FString(UE::CaptureManager::CalibTokens::NameKey), LOCTEXT("CalibrationName", "Name (used to identify a track of recorded data)") });
}

UE::CaptureManager::FIngestToken UCaptureManagerCalibrationNamingTokens::GetToken(const FString& InKey) const
{
	check(IngestCalibTokens.Contains(InKey));
	return IngestCalibTokens[InKey];
}

void UCaptureManagerCalibrationNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	Super::OnCreateDefaultTokens(OutTokens);

	for (const TPair<FString, UE::CaptureManager::FIngestToken>& Token : IngestCalibTokens)
	{
		OutTokens.Emplace(
			Token.Value.Name,
			Token.Value.Description,
			FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda(
				[Name = Token.Value.Name] {
					return FText::FromString(Name);
				}
			)
		);
	}

}

void UCaptureManagerCalibrationNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerCalibrationNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}


UCaptureManagerLensFileNamingTokens::UCaptureManagerLensFileNamingTokens()
{
	Namespace = TEXT("cml");

	IngestLensFileTokens.Emplace(FString(UE::CaptureManager::LensFileTokens::CameraNameKey), { FString(UE::CaptureManager::LensFileTokens::CameraNameKey), LOCTEXT("CalibrationCameraName", "Camera Name") });
}

UE::CaptureManager::FIngestToken UCaptureManagerLensFileNamingTokens::GetToken(const FString& InKey) const
{
	check(IngestLensFileTokens.Contains(InKey));
	return IngestLensFileTokens[InKey];
}

void UCaptureManagerLensFileNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	Super::OnCreateDefaultTokens(OutTokens);

	for (const TPair<FString, UE::CaptureManager::FIngestToken>& Token : IngestLensFileTokens)
	{
		OutTokens.Emplace(
			Token.Value.Name,
			Token.Value.Description,
			FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda(
				[Name = Token.Value.Name] {
					return FText::FromString(Name);
				}
			)
		);
	}

}

void UCaptureManagerLensFileNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerLensFileNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}

#undef LOCTEXT_NAMESPACE
