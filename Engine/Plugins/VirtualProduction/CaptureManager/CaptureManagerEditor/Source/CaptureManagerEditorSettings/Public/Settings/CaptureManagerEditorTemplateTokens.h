// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"

#include "CaptureManagerEditorTemplateTokens.generated.h"

#define UE_API CAPTUREMANAGEREDITORSETTINGS_API

namespace UE::CaptureManager
{

struct FIngestToken
{
	FString Name;
	FText Description;
};

namespace GeneralTokens
{
static constexpr FStringView IdKey = TEXTVIEW("id");
static constexpr FStringView DeviceKey = TEXTVIEW("device");
static constexpr FStringView SlateKey = TEXTVIEW("slate");
static constexpr FStringView TakeKey = TEXTVIEW("take");
}

namespace VideoTokens
{
static constexpr FStringView NameKey = TEXTVIEW("name");
static constexpr FStringView FrameRateKey = TEXTVIEW("frameRate");
}

namespace AudioTokens
{
static constexpr FStringView NameKey = TEXTVIEW("name");
}

namespace CalibTokens
{
static constexpr FStringView NameKey = TEXTVIEW("name");
}

namespace LensFileTokens
{
static constexpr FStringView CameraNameKey = TEXTVIEW("cameraName");
}
}

UCLASS(MinimalAPI)
class UCaptureManagerIngestNamingTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UE_API UCaptureManagerIngestNamingTokens();

	UE_API UE::CaptureManager::FIngestToken GetToken(const FString& InKey) const;

protected:
	// ~Begin UNamingTokens
	UE_API virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	UE_API virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	UE_API virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

private:
	TMap<FString, UE::CaptureManager::FIngestToken> IngestGeneralTokens;
};

UCLASS(MinimalAPI)
class UCaptureManagerVideoNamingTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UE_API UCaptureManagerVideoNamingTokens();

	UE_API UE::CaptureManager::FIngestToken GetToken(const FString& InKey) const;

protected:
	// ~Begin UNamingTokens
	UE_API virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	UE_API virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	UE_API virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

private:
	TMap<FString, UE::CaptureManager::FIngestToken> IngestVideoTokens;
};

UCLASS(MinimalAPI)
class UCaptureManagerAudioNamingTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UE_API UCaptureManagerAudioNamingTokens();

	UE_API UE::CaptureManager::FIngestToken GetToken(const FString& InKey) const;

protected:
	// ~Begin UNamingTokens
	UE_API virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	UE_API virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	UE_API virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

private:
	TMap<FString, UE::CaptureManager::FIngestToken> IngestAudioTokens;
};

UCLASS(MinimalAPI)
class UCaptureManagerCalibrationNamingTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UE_API UCaptureManagerCalibrationNamingTokens();

	UE_API UE::CaptureManager::FIngestToken GetToken(const FString& InKey) const;

protected:
	// ~Begin UNamingTokens
	UE_API virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	UE_API virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	UE_API virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

private:
	TMap<FString, UE::CaptureManager::FIngestToken> IngestCalibTokens;
};

UCLASS(MinimalAPI)
class UCaptureManagerLensFileNamingTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UE_API UCaptureManagerLensFileNamingTokens();

	UE_API UE::CaptureManager::FIngestToken GetToken(const FString& InKey) const;

protected:
	// ~Begin UNamingTokens
	UE_API virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	UE_API virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	UE_API virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

private:
	TMap<FString, UE::CaptureManager::FIngestToken> IngestLensFileTokens;
};

#undef UE_API
