// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using UnrealBuildTool.Rules;

public class CaptureManagerIngestBlueprint : ModuleRules
{
	public CaptureManagerIngestBlueprint(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"CaptureDataCore",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CaptureMetadataExtraction",
			"CaptureDataConverter",
			"CaptureManagerTakeMetadata",
			"DataIngestCore",
			"DataIngestCoreEditor",
			"CaptureUtils",
			"CaptureManagerEditorSettings",
			"NamingTokens"
		});

		ShortName = "CapManIngBp";
	}
}
