// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerFindTakes.h"

#include "CaptureManagerFileExtensions.h"
#include "CaptureManagerTakeMetadata.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureManagerFindTakes, Log, All);

namespace UE::CaptureManager::FindTakes::Private
{

constexpr int32 MaxRecursionDepth = 64;

constexpr const TCHAR* CalibJson = TEXT("calib.json");

static FString GetExtensionLower(const TCHAR* InPath)
{
	return FPaths::GetExtension(InPath).ToLower();
}

// Returns true if the directory contains more than one image file (jpg/jpeg/png).
// A single image (e.g. thumbnail.jpg) does not qualify as an image sequence.
static bool DirectoryContainsImageSequence(const FString& DirPath)
{
	int32 ImageCount = 0;
	IFileManager::Get().IterateDirectory(*DirPath,
		[&ImageCount](const TCHAR* InFilePath, bool bIsDirectory) -> bool
		{
			if (!bIsDirectory && UE::CaptureManager::IsImageExtension(GetExtensionLower(InFilePath)))
			{
				++ImageCount;
				if (ImageCount > 1)
				{
					return false;
				}
			}
			return true;
		});
	return ImageCount > 1;
}

static FCaptureManagerTakeDirectoryInfo InventoryDirectory(const FString& InDirectoryPath)
{
	FCaptureManagerTakeDirectoryInfo Result;
	Result.Path = InDirectoryPath;

	bool bHasFrameLog = false;
	bool bHasVideoMetadata = false;
	int32 CptakeCount = 0;

	IFileManager::Get().IterateDirectory(*InDirectoryPath,
		[&](const TCHAR* InFilePath, bool bIsDirectory) -> bool
		{
			if (bIsDirectory)
			{
				if (DirectoryContainsImageSequence(InFilePath))
				{
					Result.ImageSeqDirs.Add(FString(InFilePath));
				}
			}
			else
			{
				const FString Ext = GetExtensionLower(InFilePath);
				const FString CleanFilename = FPaths::GetCleanFilename(InFilePath).ToLower();

				if (UE::CaptureManager::IsVideoExtension(Ext))
				{
					Result.VideoFiles.Add(FString(InFilePath));
				}
				else if (UE::CaptureManager::IsAudioExtension(Ext))
				{
					Result.AudioFiles.Add(FString(InFilePath));
				}
				else if (CleanFilename == CalibJson)
				{
					Result.CalibrationFiles.Add(FString(InFilePath));
				}
				else if (Ext == FTakeMetadata::FileExtension)
				{
					++CptakeCount;
				}

				if (CleanFilename == TEXT("frame_log.csv"))
				{
					bHasFrameLog = true;
				}

				if (CleanFilename == TEXT("video_metadata.json"))
				{
					bHasVideoMetadata = true;
				}
			}
			return true;
		});

	Result.bIsLiveLinkFace = bHasFrameLog && bHasVideoMetadata;
	Result.bIsTakeArchive = (CptakeCount == 1);

	if (CptakeCount > 1)
	{
		UE_LOG(LogCaptureManagerFindTakes, Warning,
			TEXT("Directory '%s' contains %d .cptake files; only single-archive directories are supported. Skipping."),
			*InDirectoryPath, CptakeCount);
	}

	// Take archives are self-describing - the .cptake manifest is the authoritative
	// inventory. Media arrays from filesystem scan may not match the manifest, so
	// clear them to avoid misleading callers. Pass Path to IngestTakeArchive directly.
	if (Result.bIsTakeArchive)
	{
		Result.VideoFiles.Empty();
		Result.ImageSeqDirs.Empty();
		Result.AudioFiles.Empty();
		Result.CalibrationFiles.Empty();
	}
	else
	{
		Result.VideoFiles.Sort();
		Result.ImageSeqDirs.Sort();
		Result.AudioFiles.Sort();
		Result.CalibrationFiles.Sort();
	}

	return Result;
}

static bool HasCaptureArtifacts(const FCaptureManagerTakeDirectoryInfo& Dir)
{
	return Dir.bIsTakeArchive
		|| Dir.bIsLiveLinkFace
		|| Dir.VideoFiles.Num() > 0
		|| Dir.ImageSeqDirs.Num() > 0
		|| Dir.AudioFiles.Num() > 0
		|| Dir.CalibrationFiles.Num() > 0;
}

static void FindTakesRecursive(const FString& InDirectory, TArray<FCaptureManagerTakeDirectoryInfo>& OutResults, int32 Depth = 0)
{
	if (Depth >= MaxRecursionDepth)
	{
		UE_LOG(LogCaptureManagerFindTakes, Warning,
			TEXT("FindTakeDirectories: maximum recursion depth (%d) reached at '%s'. Subdirectories will not be scanned."),
			MaxRecursionDepth, *InDirectory
		);
		return;
	}

	IFileManager::Get().IterateDirectory(*InDirectory,
		[&](const TCHAR* InPath, bool bIsDirectory) -> bool
		{
			if (!bIsDirectory)
			{
				return true;
			}

			FCaptureManagerTakeDirectoryInfo Contents = InventoryDirectory(FString(InPath));

			if (HasCaptureArtifacts(Contents))
			{
				OutResults.Add(MoveTemp(Contents));
				// Do not recurse into directories identified as takes
			}
			else
			{
				FindTakesRecursive(FString(InPath), OutResults, Depth + 1);
			}

			return true;
		});
}

} // namespace UE::CaptureManager::FindTakes::Private

namespace UE::CaptureManager
{

TArray<FCaptureManagerTakeDirectoryInfo> FindTakesInDirectory(const FString& InSearchDirectory, bool bRecursive)
{
	TArray<FCaptureManagerTakeDirectoryInfo> Result;

	if (InSearchDirectory.IsEmpty() || !FPaths::DirectoryExists(InSearchDirectory))
	{
		return Result;
	}

	FCaptureManagerTakeDirectoryInfo RootContents = FindTakes::Private::InventoryDirectory(InSearchDirectory);
	if (FindTakes::Private::HasCaptureArtifacts(RootContents))
	{
		Result.Add(MoveTemp(RootContents));
	}

	if (bRecursive)
	{
		FindTakes::Private::FindTakesRecursive(InSearchDirectory, Result);
	}

	return Result;
}

} // namespace UE::CaptureManager
