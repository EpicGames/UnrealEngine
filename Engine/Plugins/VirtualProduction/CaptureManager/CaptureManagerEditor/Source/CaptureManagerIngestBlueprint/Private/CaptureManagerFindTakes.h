// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerIngestBlueprintLibrary.h"

namespace UE::CaptureManager
{

TArray<FCaptureManagerTakeDirectoryInfo> FindTakesInDirectory(const FString& InSearchDirectory, bool bRecursive);

} // namespace UE::CaptureManager
