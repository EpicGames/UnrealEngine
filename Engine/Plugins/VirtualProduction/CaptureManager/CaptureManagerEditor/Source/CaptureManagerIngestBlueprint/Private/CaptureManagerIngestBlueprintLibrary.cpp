// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerIngestBlueprintLibrary.h"
#include "CaptureManagerIngestLog.h"

#include "Async/StopToken.h"
#include "TakeArchiveMetadataExtractor.h"
#include "MonoVideoMetadataExtractor.h"
#include "StereoVideoMetadataExtractor.h"
#include "LiveLinkFaceMetadataExtractor.h"
#include "CaptureManagerCalibrationUtils.h"
#include "CaptureManagerFindTakes.h"
#include "CaptureManagerIngestFromMetadata.h"
#include "CaptureManagerRunIngestAsync.h"
#include "CaptureManagerTakeMetadata.h"
#include "HAL/FileManager.h"

#include "Settings/CaptureManagerEditorSettings.h"
#include "Settings/CaptureManagerEditorTemplateTokens.h"
#include "NamingTokensEngineSubsystem.h"
#include "NamingTokenData.h"
#include "Engine/Engine.h"
#include "Async/HelperFunctions.h"
#include "Asset/CaptureAssetSanitization.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY(LogCaptureManagerIngest);

#define LOCTEXT_NAMESPACE "CaptureManagerIngestBlueprint"

namespace UE::CaptureManager::Ingest::Private
{

// Mirrors FLiveLinkHubImportWorker::EvaluateSettings: pre-substitutes general named args
// (id, device, slate, take) then evaluates remaining naming tokens via the subsystem.
// Safe to call from any thread - marshals the subsystem call to the game thread internally.
static FString EvaluateMediaDirectory(const FTakeMetadata& InMetadata, const UCaptureManagerEditorSettings* Settings)
{
	FStringFormatNamedArguments GeneralNamedArgs;
	if (const TObjectPtr<const UCaptureManagerIngestNamingTokens> Tokens = Settings->GetGeneralNamingTokens())
	{
		GeneralNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::IdKey)).Name, InMetadata.UniqueId);
		GeneralNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::DeviceKey)).Name, InMetadata.Device.Model);
		GeneralNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::SlateKey)).Name, InMetadata.Slate);
		GeneralNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::TakeKey)).Name, InMetadata.TakeNumber);
	}

	FString MediaDir = FString::Format(*Settings->MediaDirectory.Path, GeneralNamedArgs);

	FString Result;
	// Naming tokens subsystem consults asset registry - must run on game thread.
	CallOnGameThread(
		[&Result, &MediaDir, Settings]()
		{
			UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine ? GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>() : nullptr;
			if (!NamingTokensSubsystem)
			{
				return;
			}

			FNamingTokenFilterArgs NamingTokenArgs;
			if (const TObjectPtr<const UCaptureManagerIngestNamingTokens> Tokens = Settings->GetGeneralNamingTokens())
			{
				NamingTokenArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
			}

			Result = NamingTokensSubsystem->EvaluateTokenString(MediaDir, NamingTokenArgs).EvaluatedText.ToString();
		}
	);

	return Result;
}

static const UCaptureManagerEditorSettings* GetValidatedSettings(FText& OutErrorMessage)
{
	const UCaptureManagerEditorSettings* Settings = GetDefault<UCaptureManagerEditorSettings>();

	if (!Settings)
	{
		OutErrorMessage = LOCTEXT("SettingsNotFound", "Failed to retrieve Capture Manager editor settings");
		return nullptr;
	}

	if (Settings->MediaDirectory.Path.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("MediaDirectoryEmpty", "Media directory cannot be empty");
		return nullptr;
	}

	return Settings;
}

static FExtractionConfig MakeExtractionConfig(const UCaptureManagerEditorSettings* Settings)
{
	FExtractionConfig Params;
	Params.bUseFFprobe = Settings->bEnableThirdPartyEncoder;
	Params.FFmpegPath = Settings->ThirdPartyEncoder.FilePath;
	return Params;
}

static bool BuildPipelineOptions(
	const FTakeMetadata& InMetadata,
	const UCaptureManagerEditorSettings* Settings,
	FIngestPipelineOptions& OutOptions,
	FText& OutErrorMessage
)
{
	const FString WorkingDirectory = EvaluateMediaDirectory(InMetadata, Settings);
	if (WorkingDirectory.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("MediaDirectoryEvaluatedEmpty", "Media directory evaluated to an empty path");
		return false;
	}

	OutOptions.WorkingDirectory = WorkingDirectory;
	OutOptions.bAutoSaveAssets = Settings->bAutoSaveAssets;

	if (Settings->bEnableThirdPartyEncoder)
	{
		if (Settings->ThirdPartyEncoder.FilePath.IsEmpty())
		{
			OutErrorMessage = LOCTEXT("EncoderPathEmpty", "Third party encoder is enabled but no path is configured");
			return false;
		}

		OutOptions.EncoderPath = Settings->ThirdPartyEncoder.FilePath;
		OutOptions.CustomAudioCommandArguments = Settings->CustomAudioCommandArguments;
		OutOptions.CustomVideoCommandArguments = Settings->CustomVideoCommandArguments;
	}

	return true;
}

} // namespace UE::CaptureManager::Ingest::Private

// ---------------------------------------------------------------------------
// _Internal free functions
//
// Each contains the full body of the corresponding UFUNCTION, with an
// additional TOptional<FStopToken> parameter forwarded to IngestFromMetadata.
// UFUNCTIONs are thin wrappers calling these without a token; async lambdas
// call them with the real token from the dispatcher.
// ---------------------------------------------------------------------------

static UFootageCaptureData* IngestTakeArchive_Internal(
	const FString& InTakeArchivePath,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage,
	TOptional<UE::CaptureManager::FStopToken> InStopToken = {}
)
{
	using namespace UE::CaptureManager;

	if (InTakeArchivePath.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("TakeArchivePathEmpty", "Take archive path was empty");
		return nullptr;
	}

	FString TakeMetadataFilePath = InTakeArchivePath;

	if (FPaths::DirectoryExists(TakeMetadataFilePath))
	{
		TArray<FString> TakeFiles;
		IFileManager::Get().FindFiles(
			TakeFiles,
			*(FPaths::Combine(TakeMetadataFilePath, TEXT("*.") + FTakeMetadata::FileExtension)),
			true,
			false
		);

		if (TakeFiles.Num() == 0)
		{
			OutErrorMessage = FText::Format(
				LOCTEXT("TakeArchiveNoCptakeInDir", "Directory does not contain a .cptake file: {0}"),
				FText::FromString(TakeMetadataFilePath)
			);
			return nullptr;
		}

		if (TakeFiles.Num() > 1)
		{
			OutErrorMessage = FText::Format(
				LOCTEXT("TakeArchiveMultipleCptakeInDir", "Directory contains multiple .cptake files: {0}"),
				FText::FromString(TakeMetadataFilePath)
			);
			return nullptr;
		}

		TakeMetadataFilePath = FPaths::Combine(TakeMetadataFilePath, TakeFiles[0]);
	}

	if (!FPaths::FileExists(TakeMetadataFilePath))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("TakeArchiveFileNotFound", "Take metadata file path does not exist, or is not a regular file: {0}"),
			FText::FromString(TakeMetadataFilePath)
		);
		return nullptr;
	}

	const UCaptureManagerEditorSettings* Settings = Ingest::Private::GetValidatedSettings(OutErrorMessage);
	if (!Settings)
	{
		return nullptr;
	}

	TValueOrError<FTakeMetadata, ETakeArchiveExtractionError> TakeMetadata =
		ExtractTakeArchiveMetadata(TakeMetadataFilePath, Ingest::Private::MakeExtractionConfig(Settings));

	if (TakeMetadata.HasError())
	{
		switch (TakeMetadata.GetError())
		{
			case ETakeArchiveExtractionError::MetadataFileNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("TakeArchiveMetadataFileNotFound", "Take metadata file does not exist: {0}"),
					FText::FromString(TakeMetadataFilePath));
				break;
			case ETakeArchiveExtractionError::MetadataFormatNotRecognized:
				OutErrorMessage = FText::Format(
					LOCTEXT("TakeArchiveMetadataFormatNotRecognized", "Could not parse take metadata: {0}"),
					FText::FromString(TakeMetadataFilePath));
				break;
			default:
				OutErrorMessage = FText::Format(
					LOCTEXT("TakeArchiveExtractionFailed", "Failed to extract metadata from: {0}"),
					FText::FromString(TakeMetadataFilePath));
				break;
		}
		return nullptr;
	}

	FIngestPipelineOptions Options;
	if (!Ingest::Private::BuildPipelineOptions(TakeMetadata.GetValue(), Settings, Options, OutErrorMessage))
	{
		return nullptr;
	}


	const FString TakeOriginDirectory = FPaths::GetPath(TakeMetadataFilePath);
	return IngestFromMetadata(TakeMetadata.GetValue(), TakeOriginDirectory, InParams, Options, OutErrorMessage, InStopToken);
}

static UFootageCaptureData* IngestMonoVideo_Internal(
	const FString& InVideoFilePath,
	const FString& InAudioFilePath,
	const FString& InSlate,
	int32 InTakeNumber,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage,
	TOptional<UE::CaptureManager::FStopToken> InStopToken = {}
)
{
	using namespace UE::CaptureManager;

	if (InVideoFilePath.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("MonoVideoPathEmpty", "Video file path was empty");
		return nullptr;
	}

	const UCaptureManagerEditorSettings* Settings = Ingest::Private::GetValidatedSettings(OutErrorMessage);
	if (!Settings)
	{
		return nullptr;
	}

	FMonoVideoDescriptor Descriptor;
	Descriptor.VideoFilePath = InVideoFilePath;
	if (!InAudioFilePath.IsEmpty())
	{
		Descriptor.AudioFilePaths.Add(InAudioFilePath);
	}
	Descriptor.Slate = InSlate;
	Descriptor.TakeNumber = InTakeNumber;

	TValueOrError<FTakeMetadata, EMonoVideoExtractionError> TakeMetadata =
		ExtractMonoVideoMetadata(Descriptor, Ingest::Private::MakeExtractionConfig(Settings));

	if (TakeMetadata.HasError())
	{
		switch (TakeMetadata.GetError())
		{
			case EMonoVideoExtractionError::VideoFileNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("MonoVideoFileNotFound", "Video file does not exist: {0}"),
					FText::FromString(InVideoFilePath));
				break;
			case EMonoVideoExtractionError::UnsupportedVideoFormat:
				OutErrorMessage = FText::Format(
					LOCTEXT("MonoVideoUnsupportedFormat", "Unsupported video format '{0}'. Expected mp4 or mov"),
					FText::FromString(FPaths::GetExtension(InVideoFilePath)));
				break;
			case EMonoVideoExtractionError::AudioFileNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("MonoAudioFileNotFound", "Audio file does not exist: {0}"),
					FText::FromString(InAudioFilePath));
				break;
			default:
				OutErrorMessage = FText::Format(
					LOCTEXT("MonoVideoExtractionFailed", "Failed to extract metadata from: {0}"),
					FText::FromString(InVideoFilePath));
				break;
		}
		return nullptr;
	}

	FIngestPipelineOptions Options;
	if (!Ingest::Private::BuildPipelineOptions(TakeMetadata.GetValue(), Settings, Options, OutErrorMessage))
	{
		return nullptr;
	}


	const FString TakeOriginDirectory = FPaths::GetPath(InVideoFilePath);
	return IngestFromMetadata(TakeMetadata.GetValue(), TakeOriginDirectory, InParams, Options, OutErrorMessage, InStopToken);
}

static UFootageCaptureData* IngestStereoVideo_Internal(
	const FString& InVideoPathA,
	const FString& InVideoPathB,
	const FString& InAudioFilePath,
	const FString& InCalibrationFilePath,
	const FString& InSlate,
	int32 InTakeNumber,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage,
	TOptional<UE::CaptureManager::FStopToken> InStopToken = {}
)
{
	using namespace UE::CaptureManager;

	if (InVideoPathA.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("StereoVideoAPathEmpty", "Video A path was empty");
		return nullptr;
	}

	if (InVideoPathB.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("StereoVideoBPathEmpty", "Video B path was empty");
		return nullptr;
	}

	const UCaptureManagerEditorSettings* Settings = Ingest::Private::GetValidatedSettings(OutErrorMessage);
	if (!Settings)
	{
		return nullptr;
	}

	FStereoVideoDescriptor Descriptor;
	Descriptor.VideoPathA = InVideoPathA;
	Descriptor.VideoPathB = InVideoPathB;
	if (!InAudioFilePath.IsEmpty())
	{
		Descriptor.AudioFilePaths.Add(InAudioFilePath);
	}
	Descriptor.CalibrationFilePath = InCalibrationFilePath;
	Descriptor.Slate = InSlate;
	Descriptor.TakeNumber = InTakeNumber;

	TValueOrError<FTakeMetadata, EStereoVideoExtractionError> TakeMetadata =
		ExtractStereoVideoMetadata(Descriptor, Ingest::Private::MakeExtractionConfig(Settings));

	if (TakeMetadata.HasError())
	{
		switch (TakeMetadata.GetError())
		{
			case EStereoVideoExtractionError::VideoANotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoVideoANotFound", "Video A does not exist: {0}"),
					FText::FromString(InVideoPathA));
				break;
			case EStereoVideoExtractionError::VideoBNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoVideoBNotFound", "Video B does not exist: {0}"),
					FText::FromString(InVideoPathB));
				break;
			case EStereoVideoExtractionError::UnsupportedVideoFormatA:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoVideoUnsupportedFormatA", "Unsupported format for Video A '{0}': expected .mp4/.mov file or image-sequence folder"),
					FText::FromString(InVideoPathA));
				break;
			case EStereoVideoExtractionError::UnsupportedVideoFormatB:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoVideoUnsupportedFormatB", "Unsupported format for Video B '{0}': expected .mp4/.mov file or image-sequence folder"),
					FText::FromString(InVideoPathB));
				break;
			case EStereoVideoExtractionError::VideoTypeMismatch:
				OutErrorMessage = LOCTEXT("StereoVideoTypeMismatch",
					"Video A and Video B must be the same type: both video files or both image-sequence folders");
				break;
			case EStereoVideoExtractionError::AudioFileNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoAudioNotFound", "Audio file does not exist: {0}"),
					FText::FromString(InAudioFilePath));
				break;
			case EStereoVideoExtractionError::CalibrationFileNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoCalibrationNotFound", "Calibration file does not exist: {0}"),
					FText::FromString(InCalibrationFilePath));
				break;
			case EStereoVideoExtractionError::CalibrationFormatUnrecognized:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoCalibrationFormatUnrecognized",
						"Cannot determine calibration format for '{0}'. Expected OpenCV or Unreal JSON."),
					FText::FromString(InCalibrationFilePath));
				break;
			default:
				OutErrorMessage = FText::Format(
					LOCTEXT("StereoVideoExtractionFailed", "Failed to extract stereo video metadata from: {0}"),
					FText::FromString(InVideoPathA));
				break;
		}
		return nullptr;
	}

	FIngestPipelineOptions Options;
	if (!Ingest::Private::BuildPipelineOptions(TakeMetadata.GetValue(), Settings, Options, OutErrorMessage))
	{
		return nullptr;
	}


	const FString TakeOriginDirectory = FPaths::GetPath(InVideoPathA);
	return IngestFromMetadata(TakeMetadata.GetValue(), TakeOriginDirectory, InParams, Options, OutErrorMessage, InStopToken);
}

static UFootageCaptureData* IngestLiveLinkFace_Internal(
	const FString& InTakeDirectoryPath,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage,
	TOptional<UE::CaptureManager::FStopToken> InStopToken = {}
)
{
	using namespace UE::CaptureManager;

	if (InTakeDirectoryPath.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("LiveLinkFacePathEmpty", "Take directory path was empty");
		return nullptr;
	}

	if (!FPaths::DirectoryExists(InTakeDirectoryPath))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("LiveLinkFaceDirectoryNotFound", "Take directory does not exist: {0}"),
			FText::FromString(InTakeDirectoryPath)
		);
		return nullptr;
	}

	const UCaptureManagerEditorSettings* Settings = Ingest::Private::GetValidatedSettings(OutErrorMessage);
	if (!Settings)
	{
		return nullptr;
	}

	TValueOrError<FTakeMetadata, ELiveLinkFaceExtractionError> TakeMetadata =
		ExtractLiveLinkFaceMetadata(InTakeDirectoryPath, Ingest::Private::MakeExtractionConfig(Settings));

	if (TakeMetadata.HasError())
	{
		switch (TakeMetadata.GetError())
		{
			case ELiveLinkFaceExtractionError::DirectoryNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("LiveLinkFaceExtractionDirectoryNotFound", "Take directory does not exist: {0}"),
					FText::FromString(InTakeDirectoryPath));
				break;
			case ELiveLinkFaceExtractionError::MetadataFileNotFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("LiveLinkFaceMetadataFileNotFound", "No take metadata found in directory: {0}"),
					FText::FromString(InTakeDirectoryPath));
				break;
			case ELiveLinkFaceExtractionError::MetadataFormatNotRecognized:
				OutErrorMessage = FText::Format(
					LOCTEXT("LiveLinkFaceMetadataFormatNotRecognized", "Could not parse take metadata in directory: {0}"),
					FText::FromString(InTakeDirectoryPath));
				break;
			case ELiveLinkFaceExtractionError::MultipleMetadataFilesFound:
				OutErrorMessage = FText::Format(
					LOCTEXT("LiveLinkFaceMultipleMetadataFiles", "Multiple .cptake files found in directory: {0}"),
					FText::FromString(InTakeDirectoryPath));
				break;
			default:
				OutErrorMessage = FText::Format(
					LOCTEXT("LiveLinkFaceExtractionFailed", "Failed to extract metadata from: {0}"),
					FText::FromString(InTakeDirectoryPath));
				break;
		}
		return nullptr;
	}

	FIngestPipelineOptions Options;
	if (!Ingest::Private::BuildPipelineOptions(TakeMetadata.GetValue(), Settings, Options, OutErrorMessage))
	{
		return nullptr;
	}


	return IngestFromMetadata(TakeMetadata.GetValue(), InTakeDirectoryPath, InParams, Options, OutErrorMessage, InStopToken);
}

static UFootageCaptureData* IngestCalibration_Internal(
	const FString& InCalibrationFilePath,
	const FString& InCalibrationName,
	FText& OutErrorMessage,
	TOptional<UE::CaptureManager::FStopToken> InStopToken = {}
)
{
	using namespace UE::CaptureManager;

	if (InCalibrationFilePath.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("CalibrationPathEmpty", "Calibration file path was empty");
		return nullptr;
	}

	if (!FPaths::FileExists(InCalibrationFilePath))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("CalibrationFileNotFound", "Calibration file does not exist: {0}"),
			FText::FromString(InCalibrationFilePath));
		return nullptr;
	}

	if (InCalibrationName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("CalibrationNameEmpty", "Calibration name was empty");
		return nullptr;
	}

	const UCaptureManagerEditorSettings* Settings = Ingest::Private::GetValidatedSettings(OutErrorMessage);
	if (!Settings)
	{
		return nullptr;
	}

	const FString CalibrationFormat = DetectCalibrationFormat(InCalibrationFilePath);
	if (CalibrationFormat.IsEmpty())
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("CalibrationFormatUnrecognized",
				"Cannot determine calibration format for '{0}'. Expected OpenCV or Unreal JSON."),
			FText::FromString(InCalibrationFilePath));
		return nullptr;
	}

	FTakeMetadata Metadata;
	Metadata.Version.Major = 4;
	Metadata.Version.Minor = 1;
	FString Slate = InCalibrationName;
	SanitizePackagePath(Slate, TEXT('_'));
	Metadata.Slate = MoveTemp(Slate);
	Metadata.TakeNumber = 1;
	Metadata.UniqueId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
	Metadata.Device.Name = TEXT("Calibration");
	Metadata.Device.Type = TEXT("Calibration");
	Metadata.Device.Model = TEXT("Calibration");

	FTakeMetadata::FCalibration Calibration;
	Calibration.Name = InCalibrationName;
	Calibration.Path = InCalibrationFilePath;
	Calibration.Format = CalibrationFormat;
	Metadata.Calibration.Add(MoveTemp(Calibration));

	FIngestPipelineOptions Options;
	if (!Ingest::Private::BuildPipelineOptions(Metadata, Settings, Options, OutErrorMessage))
	{
		return nullptr;
	}


	const FString TakeOriginDirectory = FPaths::GetPath(InCalibrationFilePath);
	const FCaptureManagerConversionParams DefaultParams;
	return IngestFromMetadata(Metadata, TakeOriginDirectory, DefaultParams, Options, OutErrorMessage, InStopToken);
}

// ---------------------------------------------------------------------------
// UFUNCTION wrappers
// ---------------------------------------------------------------------------

UFootageCaptureData* UCaptureManagerIngestBlueprintLibrary::IngestTakeArchiveSync(
	const FString& InTakeArchivePath,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage
)
{
	return IngestTakeArchive_Internal(InTakeArchivePath, InParams, OutErrorMessage);
}

int32 UCaptureManagerIngestBlueprintLibrary::IngestTakeArchive(
	FString InTakeArchivePath,
	FCaptureManagerConversionParams InParams,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
)
{
	return UE::CaptureManager::RunIngestAsync(
		ECaptureManagerIngestType::TakeArchive,
		[Path = MoveTemp(InTakeArchivePath), Params = MoveTemp(InParams)]
		(FText& OutError, int32 IngestId, const UE::CaptureManager::FStopToken& StopToken)
		{
			UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingesting as TakeArchive (IngestId=%d): %s"), IngestId, *Path);
			return IngestTakeArchive_Internal(Path, Params, OutError, StopToken);
		},
		MoveTemp(OnSuccess),
		MoveTemp(OnFailure)
	);
}

UFootageCaptureData* UCaptureManagerIngestBlueprintLibrary::IngestMonoVideoSync(
	const FString& InVideoFilePath,
	const FString& InAudioFilePath,
	const FString& InSlate,
	int32 InTakeNumber,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage
)
{
	return IngestMonoVideo_Internal(InVideoFilePath, InAudioFilePath, InSlate, InTakeNumber, InParams, OutErrorMessage);
}

int32 UCaptureManagerIngestBlueprintLibrary::IngestMonoVideo(
	FString InVideoFilePath,
	FString InAudioFilePath,
	FString InSlate,
	int32 InTakeNumber,
	FCaptureManagerConversionParams InParams,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
)
{
	return UE::CaptureManager::RunIngestAsync(
		ECaptureManagerIngestType::MonoVideo,
		[VideoPath = MoveTemp(InVideoFilePath), AudioPath = MoveTemp(InAudioFilePath),
		 Slate = MoveTemp(InSlate), TakeNumber = InTakeNumber, Params = MoveTemp(InParams)]
		(FText& OutError, int32 IngestId, const UE::CaptureManager::FStopToken& StopToken)
		{
			UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingesting as MonoVideo (IngestId=%d): %s"), IngestId, *VideoPath);
			return IngestMonoVideo_Internal(VideoPath, AudioPath, Slate, TakeNumber, Params, OutError, StopToken);
		},
		MoveTemp(OnSuccess),
		MoveTemp(OnFailure)
	);
}

UFootageCaptureData* UCaptureManagerIngestBlueprintLibrary::IngestStereoVideoSync(
	const FString& InVideoPathA,
	const FString& InVideoPathB,
	const FString& InAudioFilePath,
	const FString& InCalibrationFilePath,
	const FString& InSlate,
	int32 InTakeNumber,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage
)
{
	return IngestStereoVideo_Internal(InVideoPathA, InVideoPathB, InAudioFilePath, InCalibrationFilePath, InSlate, InTakeNumber, InParams, OutErrorMessage);
}

int32 UCaptureManagerIngestBlueprintLibrary::IngestStereoVideo(
	FString InVideoPathA,
	FString InVideoPathB,
	FString InAudioFilePath,
	FString InCalibrationFilePath,
	FString InSlate,
	int32 InTakeNumber,
	FCaptureManagerConversionParams InParams,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
)
{
	return UE::CaptureManager::RunIngestAsync(
		ECaptureManagerIngestType::StereoVideo,
		[VideoA = MoveTemp(InVideoPathA), VideoB = MoveTemp(InVideoPathB),
		 Audio = MoveTemp(InAudioFilePath), Calibration = MoveTemp(InCalibrationFilePath),
		 Slate = MoveTemp(InSlate),
		 TakeNumber = InTakeNumber, Params = MoveTemp(InParams)]
		(FText& OutError, int32 IngestId, const UE::CaptureManager::FStopToken& StopToken)
		{
			UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingesting as StereoVideo (IngestId=%d): A=%s, B=%s"), IngestId, *VideoA, *VideoB);
			return IngestStereoVideo_Internal(VideoA, VideoB, Audio, Calibration, Slate, TakeNumber, Params, OutError, StopToken);
		},
		MoveTemp(OnSuccess),
		MoveTemp(OnFailure)
	);
}

UFootageCaptureData* UCaptureManagerIngestBlueprintLibrary::IngestLiveLinkFaceSync(
	const FString& InTakeDirectoryPath,
	const FCaptureManagerConversionParams& InParams,
	FText& OutErrorMessage
)
{
	return IngestLiveLinkFace_Internal(InTakeDirectoryPath, InParams, OutErrorMessage);
}

int32 UCaptureManagerIngestBlueprintLibrary::IngestLiveLinkFace(
	FString InTakeDirectoryPath,
	FCaptureManagerConversionParams InParams,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
)
{
	return UE::CaptureManager::RunIngestAsync(
		ECaptureManagerIngestType::LiveLinkFace,
		[Path = MoveTemp(InTakeDirectoryPath), Params = MoveTemp(InParams)]
		(FText& OutError, int32 IngestId, const UE::CaptureManager::FStopToken& StopToken)
		{
			UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingesting as LiveLinkFace (IngestId=%d): %s"), IngestId, *Path);
			return IngestLiveLinkFace_Internal(Path, Params, OutError, StopToken);
		},
		MoveTemp(OnSuccess),
		MoveTemp(OnFailure)
	);
}

UFootageCaptureData* UCaptureManagerIngestBlueprintLibrary::IngestCalibrationSync(
	const FString& InCalibrationFilePath,
	const FString& InCalibrationName,
	FText& OutErrorMessage
)
{
	return IngestCalibration_Internal(InCalibrationFilePath, InCalibrationName, OutErrorMessage);
}

int32 UCaptureManagerIngestBlueprintLibrary::IngestCalibration(
	FString InCalibrationFilePath,
	FString InCalibrationName,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
)
{
	return UE::CaptureManager::RunIngestAsync(
		ECaptureManagerIngestType::Calibration,
		[CalibPath = MoveTemp(InCalibrationFilePath), CalibName = MoveTemp(InCalibrationName)]
		(FText& OutError, int32 IngestId, const UE::CaptureManager::FStopToken& StopToken)
		{
			UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingesting calibration (IngestId=%d): %s"), IngestId, *CalibPath);
			return IngestCalibration_Internal(CalibPath, CalibName, OutError, StopToken);
		},
		MoveTemp(OnSuccess),
		MoveTemp(OnFailure)
	);
}

bool UCaptureManagerIngestBlueprintLibrary::CancelIngest(int32 InIngestId)
{
	return UE::CaptureManager::CancelIngest(InIngestId);
}

TArray<FCaptureManagerTakeDirectoryInfo> UCaptureManagerIngestBlueprintLibrary::FindTakeDirectories(
	const FString& InSearchDirectory,
	bool bRecursive
)
{
	return UE::CaptureManager::FindTakesInDirectory(InSearchDirectory, bRecursive);
}

#undef LOCTEXT_NAMESPACE
