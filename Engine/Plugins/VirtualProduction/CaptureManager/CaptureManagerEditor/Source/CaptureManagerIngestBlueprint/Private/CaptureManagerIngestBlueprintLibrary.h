// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "CaptureData.h"
#include "CaptureManagerConversionTypes.h"

#include "CaptureManagerIngestBlueprintLibrary.generated.h"

/** Describes which ingest path produced a result. Returned in async ingest callbacks as context. */
UENUM(BlueprintType)
enum class ECaptureManagerIngestType : uint8
{
	MonoVideo        UMETA(DisplayName = "Mono Video"),
	StereoVideo      UMETA(DisplayName = "Stereo Video"),
	LiveLinkFace     UMETA(DisplayName = "LiveLink Face"),
	TakeArchive      UMETA(DisplayName = "Take Archive"),
	Calibration      UMETA(DisplayName = "Calibration"),
};

UENUM(BlueprintType)
enum class ECaptureManagerImageFormat : uint8
{
	Png = 0 UMETA(DisplayName = "PNG"),
	Jpg = 1 UMETA(DisplayName = "JPG")
};

UENUM(BlueprintType)
enum class ECaptureManagerAudioFormat : uint8
{
	Wav = 0 UMETA(DisplayName = "WAV")
};

USTRUCT(BlueprintType)
struct FCaptureManagerConversionParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	ECaptureManagerImageFormat ImageFormat = ECaptureManagerImageFormat::Png;

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	ECaptureManagerAudioFormat AudioFormat = ECaptureManagerAudioFormat::Wav;

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	FString ImageFilePrefix = TEXT("frame");

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	FString AudioFilePrefix = TEXT("audio");

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	FString DepthFilePrefix = TEXT("depth");

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	FString CalibrationFilePrefix = TEXT("calibration");

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	ECaptureManagerPixelFormat PixelFormat = ECaptureManagerPixelFormat::U8_BGRA;

	UPROPERTY(BlueprintReadWrite, Category = Ingest)
	ECaptureManagerRotation Rotation = ECaptureManagerRotation::Auto;

};

/** Inventory of capture-related files found in a single directory. */
USTRUCT(BlueprintType)
struct FCaptureManagerTakeDirectoryInfo
{
	GENERATED_BODY()

	/** Absolute path to the directory. */
	UPROPERTY(BlueprintReadOnly, Category = "Ingest")
	FString Path;

	/** True if the directory contains a .cptake file. */
	UPROPERTY(BlueprintReadOnly, Category = "Ingest")
	bool bIsTakeArchive = false;

	/** True if the directory contains frame_log.csv and video_metadata.json (LiveLink Face capture). */
	UPROPERTY(BlueprintReadOnly, Category = "Ingest")
	bool bIsLiveLinkFace = false;

	/** Full paths of video files (.mp4, .mov, .avi, .mkv, .webm) found in the directory. */
	UPROPERTY(BlueprintReadOnly, Category = "Ingest")
	TArray<FString> VideoFiles;

	/** Full paths of subdirectories containing image sequences (.jpg, .jpeg, .png). */
	UPROPERTY(BlueprintReadOnly, Category = "Ingest")
	TArray<FString> ImageSeqDirs;

	/** Full paths of audio files (.wav, .mp3, .flac, .m4a, .aac) found in the directory. */
	UPROPERTY(BlueprintReadOnly, Category = "Ingest")
	TArray<FString> AudioFiles;

	/** Full paths of calibration files found in the directory. Currently only files named "calib.json" are detected. */
	UPROPERTY(BlueprintReadOnly, Category = "Ingest")
	TArray<FString> CalibrationFiles;
};

DECLARE_DYNAMIC_DELEGATE_ThreeParams(FCaptureManagerIngestSuccess, int32, IngestId, ECaptureManagerIngestType, IngestType, UFootageCaptureData*, FootageCaptureData);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FCaptureManagerIngestFailed, int32, IngestId, ECaptureManagerIngestType, IngestType, FText, ErrorMessage);

UCLASS()
class UCaptureManagerIngestBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Import a take archive (.cptake). Accepts a directory path or a direct path to the .cptake file.
	 * Slate, take number, and device info are read from the archive metadata.
	 *
	 * @param TakeArchivePath		Path to the take directory or .cptake file. If a directory is given, searches for a .cptake file automatically.
	 * @param Params				Conversion settings (pixel format, rotation, output formats).
	 * @param OutErrorMessage		Set on failure; describes what went wrong.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest|Blocking", meta = (DisplayName = "Ingest Take Archive", ReturnDisplayName = "FootageCaptureData", Keywords = "import"))
	static UFootageCaptureData* IngestTakeArchiveSync(
		const FString& TakeArchivePath,
		const FCaptureManagerConversionParams& Params,
		FText& OutErrorMessage
	);

	/**
	 * Import a take archive (.cptake) asynchronously. Accepts a directory path or a direct path to the .cptake file.
	 * Slate, take number, and device info are read from the archive metadata.
	 *
	 * @param TakeArchivePath		Path to the take directory or .cptake file. If a directory is given, searches for a .cptake file automatically.
	 * @param Params				Conversion settings (pixel format, rotation, output formats).
	 * @param OnSuccess				Called on the game thread when the import completes successfully.
	 * @param OnFailure				Called on the game thread if the import fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest", meta = (DisplayName = "Ingest Take Archive", ReturnDisplayName = "IngestId", Keywords = "import"))
	static int32 IngestTakeArchive(
		FString TakeArchivePath,
		FCaptureManagerConversionParams Params,
		FCaptureManagerIngestSuccess OnSuccess,
		FCaptureManagerIngestFailed OnFailure
	);

	/**
	 * Import a single-camera video file (.mp4 or .mov).
	 *
	 * @param VideoFilePath			Path to the video file.
	 * @param AudioFilePath			Optional. Path to a separate audio file. Leave empty to use embedded audio if present.
	 * @param Slate					Optional. Slate name for the imported take. Derived from the video filename if empty.
	 * @param TakeNumber			Optional. Take number. Defaults to 1 if less than 1.
	 * @param Params				Conversion settings (pixel format, rotation, output formats).
	 * @param OutErrorMessage		Set on failure; describes what went wrong.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest|Blocking", meta = (DisplayName = "Ingest Mono Video", ReturnDisplayName = "FootageCaptureData", InTakeNumber = "1", Keywords = "import"))
	static UFootageCaptureData* IngestMonoVideoSync(
		const FString& VideoFilePath,
		const FString& AudioFilePath,
		const FString& Slate,
		int32 TakeNumber,
		const FCaptureManagerConversionParams& Params,
		FText& OutErrorMessage
	);

	/**
	 * Import a single-camera video file (.mp4 or .mov) asynchronously.
	 *
	 * @param VideoFilePath			Path to the video file.
	 * @param AudioFilePath			Optional. Path to a separate audio file. Leave empty to use embedded audio if present.
	 * @param Slate					Optional. Slate name for the imported take. Derived from the video filename if empty.
	 * @param TakeNumber			Optional. Take number. Defaults to 1 if less than 1.
	 * @param Params				Conversion settings (pixel format, rotation, output formats).
	 * @param OnSuccess				Called on the game thread when the import completes successfully.
	 * @param OnFailure				Called on the game thread if the import fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest", meta = (DisplayName = "Ingest Mono Video", ReturnDisplayName = "IngestId", InTakeNumber = "1", Keywords = "import"))
	static int32 IngestMonoVideo(
		FString VideoFilePath,
		FString AudioFilePath,
		FString Slate,
		int32 TakeNumber,
		FCaptureManagerConversionParams Params,
		FCaptureManagerIngestSuccess OnSuccess,
		FCaptureManagerIngestFailed OnFailure
	);

	/**
	 * Import a dual-camera stereo video. Pass two video files (.mp4/.mov) or two image-sequence folders.
	 *
	 * @param VideoPathA				Path to the first video file or image-sequence folder. Becomes the default primary camera for MHA processing.
	 * @param VideoPathB				Path to the second video file or image-sequence folder. Must be the same type as the first video (both files, or both folders). Becomes the secondary view, resolved automatically.
	 * @param AudioFilePath				Optional. Path to a dedicated audio file. Leave empty if there is no separate audio.
	 * @param CalibrationFilePath		Optional. Path to a calibration file (.json). Format (OpenCV or Unreal) is auto-detected from JSON structure. Leave empty if not available.
	 * @param Slate						Optional. Slate name for the imported take. Derived from the first video's filename if empty.
	 * @param TakeNumber				Optional. Take number. Defaults to 1 if less than 1.
	 * @param Params					Conversion settings (pixel format, rotation, output formats).
	 * @param OutErrorMessage			Set on failure; describes what went wrong.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest|Blocking", meta = (DisplayName = "Ingest Stereo Video", ReturnDisplayName = "FootageCaptureData", InTakeNumber = "1", Keywords = "import"))
	static UFootageCaptureData* IngestStereoVideoSync(
		const FString& VideoPathA,
		const FString& VideoPathB,
		const FString& AudioFilePath,
		const FString& CalibrationFilePath,
		const FString& Slate,
		int32 TakeNumber,
		const FCaptureManagerConversionParams& Params,
		FText& OutErrorMessage
	);

	/**
	 * Import a dual-camera stereo video asynchronously. Pass two video files (.mp4/.mov) or two image-sequence folders.
	 *
	 * @param VideoPathA				Path to the first video file or image-sequence folder. Becomes the default primary camera for MHA processing.
	 * @param VideoPathB				Path to the second video file or image-sequence folder. Must be the same type as the first video (both files, or both folders). Becomes the secondary view, resolved automatically.
	 * @param AudioFilePath				Optional. Path to a dedicated audio file. Leave empty if there is no separate audio.
	 * @param CalibrationFilePath		Optional. Path to a calibration file (.json). Format (OpenCV or Unreal) is auto-detected from JSON structure. Leave empty if not available.
	 * @param Slate						Optional. Slate name for the imported take. Derived from the first video's filename if empty.
	 * @param TakeNumber				Optional. Take number. Defaults to 1 if less than 1.
	 * @param Params					Conversion settings (pixel format, rotation, output formats).
	 * @param OnSuccess					Called on the game thread when the import completes successfully.
	 * @param OnFailure					Called on the game thread if the import fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest", meta = (DisplayName = "Ingest Stereo Video", ReturnDisplayName = "IngestId", InTakeNumber = "1", Keywords = "import"))
	static int32 IngestStereoVideo(
		FString VideoPathA,
		FString VideoPathB,
		FString AudioFilePath,
		FString CalibrationFilePath,
		FString Slate,
		int32 TakeNumber,
		FCaptureManagerConversionParams Params,
		FCaptureManagerIngestSuccess OnSuccess,
		FCaptureManagerIngestFailed OnFailure
	);

	/**
	 * Import a LiveLink Face capture. Supports both the current .cptake format and the pre-5.6 LiveLink Face format.
	 *
	 * @param TakeDirectoryPath		Path to the take directory containing the LiveLink Face capture data.
	 * @param Params				Conversion settings (pixel format, rotation, output formats).
	 * @param OutErrorMessage		Set on failure; describes what went wrong.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest|Blocking", meta = (DisplayName = "Ingest Live Link Face", ReturnDisplayName = "FootageCaptureData", Keywords = "import livelink iphone"))
	static UFootageCaptureData* IngestLiveLinkFaceSync(
		const FString& TakeDirectoryPath,
		const FCaptureManagerConversionParams& Params,
		FText& OutErrorMessage
	);

	/**
	 * Import a LiveLink Face capture asynchronously. Supports both the current .cptake format and the pre-5.6 LiveLink Face format.
	 *
	 * @param TakeDirectoryPath		Path to the take directory containing the LiveLink Face capture data.
	 * @param Params				Conversion settings (pixel format, rotation, output formats).
	 * @param OnSuccess				Called on the game thread when the import completes successfully.
	 * @param OnFailure				Called on the game thread if the import fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest", meta = (DisplayName = "Ingest Live Link Face", ReturnDisplayName = "IngestId", Keywords = "import livelink iphone"))
	static int32 IngestLiveLinkFace(
		FString TakeDirectoryPath,
		FCaptureManagerConversionParams Params,
		FCaptureManagerIngestSuccess OnSuccess,
		FCaptureManagerIngestFailed OnFailure
	);

	/**
	 * Import a standalone calibration file (.json). Format (OpenCV or Unreal) is auto-detected.
	 * Creates a FootageCaptureData asset with no footage and one CameraCalibration entry.
	 * Access the calibration via FootageCaptureData->CameraCalibrations[0].
	 *
	 * @param CalibrationFilePath		Path to a calibration JSON file.
	 * @param CalibrationName			Name for the calibration asset. Must be unique per import - determines the output asset path. If multiple calibration files share the same filename (e.g. calib.json), consider using the parent folder name to distinguish them.
	 * @param OutErrorMessage			Set on failure; describes what went wrong.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest|Blocking", meta = (DisplayName = "Ingest Calibration", ReturnDisplayName = "FootageCaptureData", Keywords = "import"))
	static UFootageCaptureData* IngestCalibrationSync(
		const FString& CalibrationFilePath,
		const FString& CalibrationName,
		FText& OutErrorMessage
	);

	/**
	 * Import a standalone calibration file (.json) asynchronously. Format (OpenCV or Unreal) is auto-detected.
	 *
	 * @param CalibrationFilePath		Path to a calibration JSON file.
	 * @param CalibrationName			Name for the calibration asset. Must be unique per import - determines the output asset path. If multiple calibration files share the same filename (e.g. calib.json), consider using the parent folder name to distinguish them.
	 * @param OnSuccess					Called on the game thread when the import completes successfully.
	 * @param OnFailure					Called on the game thread if the import fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest", meta = (DisplayName = "Ingest Calibration", ReturnDisplayName = "IngestId", Keywords = "import"))
	static int32 IngestCalibration(
		FString CalibrationFilePath,
		FString CalibrationName,
		FCaptureManagerIngestSuccess OnSuccess,
		FCaptureManagerIngestFailed OnFailure
	);

	/**
	 * Cancel a queued or running ingest by its ID.
	 *
	 * Queued ingests are removed immediately and their OnFailure delegate fires with a
	 * cancellation message. Running ingests are signaled to stop - conversion nodes bail
	 * at their next cancellation check and the partial working directory is cleaned up.
	 *
	 * @param IngestId	The ID returned by an async ingest function.
	 * @return				True if the IngestId was found and cancellation was initiated.
	 *						False if it was already completed, already canceled, or invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest", meta = (Keywords = "abort stop cancel"))
	static bool CancelIngest(int32 IngestId);

	/**
	 * Scans a directory for capture-related content and returns an inventory of what was found.
	 *
	 * Returns one entry per directory that contains at least one recognized capture artifact
	 * (video, image sequence, audio, calibration, or .cptake archive).
	 *
	 * File arrays are populated by filesystem scan only, not by parsing .cptake metadata.
	 * When bIsTakeArchive is true, the file arrays are empty - the .cptake manifest is
	 * the authoritative inventory. Pass the directory path to IngestTakeArchive directly.
	 *
	 * @param SearchDirectory		Root directory to search.
	 * @param bRecursive			If true, recurse into subdirectories. Recursion stops
	 *								at directories that contain capture artifacts.
	 */
	UFUNCTION(BlueprintCallable, Category = "CaptureManager|Ingest", meta = (ReturnDisplayName = "TakeDirectories", Keywords = "scan"))
	static TArray<FCaptureManagerTakeDirectoryInfo> FindTakeDirectories(
		const FString& SearchDirectory,
		bool bRecursive = true
	);
};

