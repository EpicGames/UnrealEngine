// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerIngestBlueprintModule.h"
#include "CaptureManagerIngestDispatcher.h"

#include "Modules/ModuleManager.h"

void FCaptureManagerIngestBlueprintModule::StartupModule()
{
	Dispatcher = MakeUnique<UE::CaptureManager::FCaptureManagerIngestDispatcher>();
}

void FCaptureManagerIngestBlueprintModule::ShutdownModule()
{
	if (Dispatcher)
	{
		Dispatcher->Shutdown();
	}
	Dispatcher.Reset();
}

UE::CaptureManager::FCaptureManagerIngestDispatcher& FCaptureManagerIngestBlueprintModule::GetDispatcher()
{
	return *Dispatcher;
}

IMPLEMENT_MODULE(FCaptureManagerIngestBlueprintModule, CaptureManagerIngestBlueprint)
