// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

namespace UE::CaptureManager
{
class FCaptureManagerIngestDispatcher;
}

class FCaptureManagerIngestBlueprintModule : public IModuleInterface
{
public:
	void StartupModule() override;
	void ShutdownModule() override;

	UE::CaptureManager::FCaptureManagerIngestDispatcher& GetDispatcher();

private:
	TUniquePtr<UE::CaptureManager::FCaptureManagerIngestDispatcher> Dispatcher;
};
