// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerIngestDispatcher.h"
#include "CaptureManagerIngestLog.h"

#include "Containers/Ticker.h"
#include "Settings/CaptureManagerEditorSettings.h"
#include "Tasks/Task.h"
#include "UObject/StrongObjectPtr.h"

#define LOCTEXT_NAMESPACE "CaptureManagerIngestDispatcher"

namespace UE::CaptureManager
{

void FCaptureManagerIngestDispatcher::Shutdown()
{
	TArray<UE::Tasks::FTask> TasksToWait;

	{
		FScopeLock ScopeLock(&Lock);
		bShuttingDown = true;

		for (TPair<int32, TSharedPtr<FStopRequester>>& Pair : ActiveRequesters)
		{
			Pair.Value->RequestStop();
		}

		Queue.Empty();

		TasksToWait.Reserve(ActiveTasks.Num());
		for (TPair<int32, UE::Tasks::FTask>& Pair : ActiveTasks)
		{
			TasksToWait.Add(Pair.Value);
		}
	}

	for (UE::Tasks::FTask& Task : TasksToWait)
	{
		Task.Wait();
	}
}

int32 FCaptureManagerIngestDispatcher::Enqueue(
	ECaptureManagerIngestType IngestType,
	TFunction<UFootageCaptureData*(FText&, int32, const FStopToken&)> SyncIngestFn,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
)
{
	FScopeLock ScopeLock(&Lock);

	if (bShuttingDown)
	{
		return INDEX_NONE;
	}
	const int32 Id = NextIngestId++;
	FPendingIngest Pending;
	Pending.IngestId = Id;
	Pending.IngestType = IngestType;
	Pending.SyncIngestFn = MoveTemp(SyncIngestFn);
	Pending.OnSuccess = MoveTemp(OnSuccess);
	Pending.OnFailure = MoveTemp(OnFailure);
	Pending.StopRequester = MakeShared<FStopRequester>();
	Queue.Add(MoveTemp(Pending));
	TryLaunchNext();
	return Id;
}

bool FCaptureManagerIngestDispatcher::CancelIngest(int32 InIngestId)
{
	FPendingIngest CanceledIngest;
	bool bFoundInQueue = false;
	bool bCanceled = false;

	{
		FScopeLock ScopeLock(&Lock);

		const int32 QueueIndex = Queue.IndexOfByPredicate(
			[InIngestId](const FPendingIngest& P) { return P.IngestId == InIngestId; });

		if (QueueIndex != INDEX_NONE)
		{
			CanceledIngest = MoveTemp(Queue[QueueIndex]);
			Queue.RemoveAt(QueueIndex, EAllowShrinking::No);
			bFoundInQueue = true;
			bCanceled = true;
		}
		else if (TSharedPtr<FStopRequester>* Requester = ActiveRequesters.Find(InIngestId))
		{
			(*Requester)->RequestStop();
			bCanceled = true;
		}
	}

	if (bFoundInQueue)
	{
		UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingest canceled from queue (IngestId=%d, %s)"),
			CanceledIngest.IngestId, *UEnum::GetValueAsString(CanceledIngest.IngestType));

		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda(
				[OnFailure = MoveTemp(CanceledIngest.OnFailure),
				 IngestId = CanceledIngest.IngestId,
				 IngestType = CanceledIngest.IngestType](float)
				{
					check(IsInGameThread());
					OnFailure.ExecuteIfBound(IngestId, IngestType,
						LOCTEXT("IngestCanceledByUser", "Ingest was canceled"));
					return false;
				}
			)
		);
	}
	else if (bCanceled)
	{
		UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingest cancellation requested (IngestId=%d, running)"), InIngestId);
	}

	return bCanceled;
}

void FCaptureManagerIngestDispatcher::TryLaunchNext()
{
	const UCaptureManagerEditorSettings* Settings = GetDefault<UCaptureManagerEditorSettings>();
	const int32 MaxActive = Settings ? Settings->MaxConcurrentIngests : 1;

	while (ActiveCount < MaxActive && !Queue.IsEmpty())
	{
		FPendingIngest Pending = MoveTemp(Queue[0]);
		Queue.RemoveAt(0, EAllowShrinking::No);
		++ActiveCount;

		ActiveRequesters.Add(Pending.IngestId, Pending.StopRequester);

		const int32 LaunchedIngestId = Pending.IngestId;

		UE::Tasks::FTask Task = UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[this, Pending = MoveTemp(Pending)]() mutable
			{
				const FStopToken Token = Pending.StopRequester->CreateToken();

				FText ErrorMessage;
				const ECaptureManagerIngestType IngestType = Pending.IngestType;
				TStrongObjectPtr<UFootageCaptureData> FootageCaptureData(Pending.SyncIngestFn(ErrorMessage, Pending.IngestId, Token));

				if (FootageCaptureData)
				{
					UE_LOG(LogCaptureManagerIngest, Display, TEXT("Ingest complete (IngestId=%d, %s): Success"),
						Pending.IngestId, *UEnum::GetValueAsString(IngestType));
				}
				else
				{
					UE_LOG(LogCaptureManagerIngest, Error, TEXT("Ingest failed (IngestId=%d, %s): %s"),
						Pending.IngestId, *UEnum::GetValueAsString(IngestType), *ErrorMessage.ToString());
				}

				// Dispatch the callback via the core ticker rather than
				// CallOnGameThread (task graph). The ticker fires during
				// FEngineLoop::Tick before world tick / task graph processing,
				// so the Blueprint handler runs in a clean game-thread context
				// where synchronous waits (e.g. Interchange ImportSync for DNA
				// import) can pump the task queue without re-entrancy deadlock.
				FTSTicker::GetCoreTicker().AddTicker(
					FTickerDelegate::CreateLambda(
						[FootageCaptureData = MoveTemp(FootageCaptureData),
						 OnSuccess = MoveTemp(Pending.OnSuccess),
						 OnFailure = MoveTemp(Pending.OnFailure),
						 ErrorMessage = MoveTemp(ErrorMessage),
						 IngestId = Pending.IngestId,
						 IngestType](float)
						{
							check(IsInGameThread());

							if (FootageCaptureData)
							{
								OnSuccess.ExecuteIfBound(IngestId, IngestType, FootageCaptureData.Get());
							}
							else
							{
								OnFailure.ExecuteIfBound(IngestId, IngestType, ErrorMessage);
							}

							return false;
						}
					)
				);

				FScopeLock ScopeLock(&Lock);
				ActiveRequesters.Remove(Pending.IngestId);
				ActiveTasks.Remove(Pending.IngestId);
				--ActiveCount;
				TryLaunchNext();
			},
			UE::Tasks::ETaskPriority::BackgroundNormal
		);

		ActiveTasks.Add(LaunchedIngestId, MoveTemp(Task));
	}
}

} // namespace UE::CaptureManager

#undef LOCTEXT_NAMESPACE
