// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/StopToken.h"
#include "CaptureManagerIngestBlueprintLibrary.h"
#include "HAL/CriticalSection.h"
#include "Tasks/Task.h"
#include "Templates/Function.h"

class UFootageCaptureData;

namespace UE::CaptureManager
{

/**
 * Deferred-launch dispatcher for Blueprint ingest async calls.
 *
 * Maintains a FIFO queue of pending ingest requests and launches at most
 * MaxConcurrentIngests tasks at once (read from
 * UCaptureManagerEditorSettings at each launch). Tasks are only created when
 * a slot is free - no worker threads are ever blocked waiting for a slot.
 */
class FCaptureManagerIngestDispatcher
{
public:
	/**
	 * Drain active tasks and reject future work. Signals all running ingests
	 * to stop via their FStopToken, discards queued items, then blocks until
	 * every in-flight task has completed. Called from ShutdownModule on the
	 * game thread.
	 */
	void Shutdown();

	int32 Enqueue(
		ECaptureManagerIngestType IngestType,
		TFunction<UFootageCaptureData*(FText&, int32, const FStopToken&)> SyncIngestFn,
		FCaptureManagerIngestSuccess OnSuccess,
		FCaptureManagerIngestFailed OnFailure
	);

	/**
	 * Cancel a queued or running ingest.
	 *
	 * Queued ingests are removed from the queue and their OnFailure delegate is
	 * fired on the game thread with a "canceled" message. Running ingests are
	 * signaled via FStopToken - conversion nodes will bail at their next check.
	 *
	 * @return true if the IngestId was found (queued or running), false if it
	 *         was already completed, already canceled, or never existed.
	 */
	bool CancelIngest(int32 InIngestId);

private:
	struct FPendingIngest
	{
		int32 IngestId;
		ECaptureManagerIngestType IngestType;
		TFunction<UFootageCaptureData*(FText&, int32, const FStopToken&)> SyncIngestFn;
		FCaptureManagerIngestSuccess OnSuccess;
		FCaptureManagerIngestFailed OnFailure;
		TSharedPtr<FStopRequester> StopRequester;
	};

	// Must be called under Lock.
	void TryLaunchNext();

	FCriticalSection Lock;
	TArray<FPendingIngest> Queue;
	int32 ActiveCount = 0;
	int32 NextIngestId = 1;

	// Maps IngestId to the stop requester for currently running ingests.
	// Entries are added in TryLaunchNext before the task launches and removed
	// inside the task lambda under Lock before ActiveCount is decremented.
	// The TSharedPtr is also held by the task lambda, so the requester remains
	// valid for the entire task lifetime.
	TMap<int32, TSharedPtr<FStopRequester>> ActiveRequesters;

	// Task handles for currently running ingests, keyed by IngestId.
	// Mirrors ActiveRequesters lifecycle. Used by Shutdown() to wait for
	// in-flight tasks before the dispatcher is destroyed.
	TMap<int32, UE::Tasks::FTask> ActiveTasks;

	bool bShuttingDown = false;
};

} // namespace UE::CaptureManager
