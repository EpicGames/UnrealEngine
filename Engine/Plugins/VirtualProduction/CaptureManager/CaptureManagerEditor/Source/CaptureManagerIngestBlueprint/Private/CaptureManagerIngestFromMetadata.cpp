// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerIngestFromMetadata.h"

#include "CaptureManagerIngestBlueprintLibrary.h"
#include "CaptureDataConverter.h"
#include "CaptureManagerConversionTypes.h"
#include "CaptureManagerIngestPreparation.h"
#include "IngestCaptureData.h"

#include "Settings/CaptureManagerEditorTemplateTokens.h"

#include "NamingTokensEngineSubsystem.h"
#include "Engine/Engine.h"
#include "NamingTokenData.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Async/HelperFunctions.h"

#define LOCTEXT_NAMESPACE "CaptureManagerIngestBlueprint"

namespace UE::CaptureManager::Ingest::Private
{

static FStringView ToFormatExtension(ECaptureManagerImageFormat InFormat)
{
	switch (InFormat)
	{
	case ECaptureManagerImageFormat::Jpg: return TEXTVIEW("jpg");
	case ECaptureManagerImageFormat::Png:
	default: return TEXTVIEW("png");
	}
}

static FStringView ToFormatExtension(ECaptureManagerAudioFormat InFormat)
{
	switch (InFormat)
	{
	case ECaptureManagerAudioFormat::Wav:
	default: return TEXTVIEW("wav");
	}
}

} // namespace UE::CaptureManager::Ingest::Private

// Resolves an encoder command pattern: falls back to InDefaultArgs when custom args are empty,
// then evaluates naming tokens in InTokenNamespace. Placeholders like {input}, {output}, {params}
// are left unresolved here - the encoder command objects substitute those at execution time.
static FString ResolveEncoderCommandPattern(
	const FString& InCustomArgs,
	FStringView InDefaultArgs,
	FStringView InTokenNamespace)
{
	FString CommandPattern = InCustomArgs.IsEmpty()
		? FString(InDefaultArgs)
		: InCustomArgs;

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(FString(InTokenNamespace));
	FilterArgs.bNativeOnly = true;

	// Naming tokens subsystem is game-thread only - marshal the call when invoked
	// from a background task (async ingest path).
	UE::CaptureManager::CallOnGameThread(
		[&CommandPattern, &FilterArgs]()
		{
			if (GEngine)
			{
				if (UNamingTokensEngineSubsystem* Sub = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>())
				{
					CommandPattern = Sub->EvaluateTokenString(CommandPattern, FilterArgs).EvaluatedText.ToString();
				}
			}
		}
	);

	return CommandPattern;
}

namespace UE::CaptureManager
{

UFootageCaptureData* IngestFromMetadata(
	const FTakeMetadata& InMetadata,
	const FString& InTakeOriginDirectory,
	const FCaptureManagerConversionParams& InParams,
	const FIngestPipelineOptions& InOptions,
	FText& OutErrorMessage,
	TOptional<FStopToken> InStopToken
)
{
	UFootageCaptureData* FootageCaptureData = nullptr;

	const FString& TakeConversionDirectory = InOptions.WorkingDirectory;

	if (IFileManager::Get().DirectoryExists(*TakeConversionDirectory))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("DirectoryAlreadyExists", "Output directory already exists: {0}"),
			FText::FromString(TakeConversionDirectory)
		);
		return FootageCaptureData;
	}

	if (!IFileManager::Get().MakeDirectory(*TakeConversionDirectory, true))
	{
		OutErrorMessage = FText::Format(
			LOCTEXT("FailedToCreateDirectory", "Failed to create output directory: {0}"),
			FText::FromString(TakeConversionDirectory)
		);
		return FootageCaptureData;
	}

	// Clean up the conversion directory on any failure path. The directory is
	// only kept when ingest succeeds (FootageCaptureData is non-null).
	ON_SCOPE_EXIT
	{
		if (!FootageCaptureData)
		{
			constexpr bool bRequireExists = false;
			constexpr bool bRecursive = true;
			IFileManager::Get().DeleteDirectory(*TakeConversionDirectory, bRequireExists, bRecursive);
		}
	};

	FCaptureDataConverterParams ConverterParams;
	ConverterParams.TakeMetadata = InMetadata;
	ConverterParams.TakeName = InMetadata.Slate + TEXT("_") + FString::FromInt(InMetadata.TakeNumber);
	ConverterParams.TakeOriginDirectory = InTakeOriginDirectory;
	ConverterParams.TakeOutputDirectory = TakeConversionDirectory;

	if (InOptions.EncoderPath.IsSet())
	{
		const FString& EncoderPath = InOptions.EncoderPath.GetValue();

		if (!FPlatformProcess::ExecProcess(*EncoderPath, TEXT("-version"), nullptr, nullptr, nullptr))
		{
			OutErrorMessage = FText::Format(
				LOCTEXT("EncoderNotAvailable", "Third party encoder not found or failed to launch: {0}"),
				FText::FromString(EncoderPath)
			);
			return FootageCaptureData;
		}

		FString AudioEncoderCommandPattern = ResolveEncoderCommandPattern(
			InOptions.CustomAudioCommandArguments,
			UE::CaptureManager::EncoderDefaults::AudioCommandArgs,
			UE::CaptureManager::AudioEncoderTokens::Namespace);
		ConverterParams.AudioEncoderConfig = FCaptureManagerEncoderConfig{ EncoderPath, MoveTemp(AudioEncoderCommandPattern) };

		FString VideoEncoderCommandPattern = ResolveEncoderCommandPattern(
			InOptions.CustomVideoCommandArguments,
			UE::CaptureManager::EncoderDefaults::VideoCommandArgs,
			UE::CaptureManager::VideoEncoderTokens::Namespace);
		ConverterParams.VideoEncoderConfig = FCaptureManagerEncoderConfig{ EncoderPath, MoveTemp(VideoEncoderCommandPattern) };
	}

	EMediaOrientation Rotation = EMediaOrientation::Original;
	if (InParams.Rotation == ECaptureManagerRotation::Auto)
	{
		if (!InMetadata.Video.IsEmpty() && InMetadata.Video[0].Orientation.IsSet())
		{
			Rotation = UE::CaptureManager::ToUprightRotation(InMetadata.Video[0].Orientation.GetValue());
		}
	}
	else
	{
		Rotation = UE::CaptureManager::ToMediaOrientation(InParams.Rotation);
	}

	FString ImageFormat(Ingest::Private::ToFormatExtension(InParams.ImageFormat));
	FCaptureConvertVideoOutputParams VideoParams;
	VideoParams.Format = ImageFormat;
	VideoParams.ImageFileName = InParams.ImageFilePrefix;
	VideoParams.OutputPixelFormat = UE::CaptureManager::ToMediaPixelFormat(InParams.PixelFormat);
	VideoParams.Rotation = Rotation;
	ConverterParams.VideoOutputParams = MoveTemp(VideoParams);

	FCaptureConvertAudioOutputParams AudioParams;
	FString AudioFormat(Ingest::Private::ToFormatExtension(InParams.AudioFormat));
	AudioParams.Format = AudioFormat;
	AudioParams.AudioFileName = InParams.AudioFilePrefix;
	ConverterParams.AudioOutputParams = MoveTemp(AudioParams);

	FCaptureConvertDepthOutputParams DepthParams;
	DepthParams.ImageFileName = InParams.DepthFilePrefix;
	DepthParams.Rotation = Rotation;
	ConverterParams.DepthOutputParams = MoveTemp(DepthParams);

	FCaptureConvertCalibrationOutputParams CalibrationParams;
	CalibrationParams.FileName = InParams.CalibrationFilePrefix;
	ConverterParams.CalibrationOutputParams = MoveTemp(CalibrationParams);

	ConverterParams.ExternalStopToken = InStopToken;

	FCaptureDataConverter Converter;
	FCaptureDataConverter::FProgressReporter ProgressReporter = FCaptureDataConverter::FProgressReporter::CreateLambda([](const double) {});
	FCaptureDataConverterResult<void> ConversionResult = Converter.Run(ConverterParams, MoveTemp(ProgressReporter));

	if (InStopToken.IsSet() && InStopToken->IsStopRequested())
	{
		OutErrorMessage = LOCTEXT("IngestCanceled", "Ingest was canceled");
		return FootageCaptureData;
	}

	if (ConversionResult.HasError())
	{
		FCaptureDataConverterError Error = ConversionResult.GetError();
		const FText Errors = FText::Join(FText::FromString(TEXT("\n")), Error.GetErrors());
		OutErrorMessage = FText::Format(LOCTEXT("ConversionFailed", "Failed to convert: {0}"), Errors);
		return FootageCaptureData;
	}

	FString ArchiveFilePath = FPaths::Combine(ConverterParams.TakeOutputDirectory, FPaths::SetExtension(TEXT("take"), FIngestCaptureData::Extension));
	IngestCaptureData::FParseResult IngestCaptureDataParseResult = IngestCaptureData::ParseFile(ArchiveFilePath);

	if (IngestCaptureDataParseResult.HasValue())
	{
		const bool bAutoSaveAssets = InOptions.bAutoSaveAssets;
		CallOnGameThread(
			[&FootageCaptureData, &IngestCaptureDataParseResult, &ConverterParams, bAutoSaveAssets, &OutErrorMessage]()
			{
				check(IsInGameThread());

				FIngestCaptureData IngestCaptureData = IngestCaptureDataParseResult.StealValue();
				FAssetNamingStrategy AssetNamingStrategy(IngestCaptureData);

				IngestCaptureData.MakePathsAbsolute(ConverterParams.TakeOutputDirectory);
				FCreateAssetsData CreateAssetsData = BuildAssetData(IngestCaptureData, AssetNamingStrategy);

				if (!CreateAssetsData.PackagePath.IsEmpty())
				{
					IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
					TArray<FAssetData> ExistingAssets;
					AssetRegistry.GetAssetsByPath(*CreateAssetsData.PackagePath, ExistingAssets);

					if (!ExistingAssets.IsEmpty())
					{
						OutErrorMessage = FText::Format(
							LOCTEXT("AssetsAlreadyExist", "Assets already exist at '{0}'."),
							FText::FromString(CreateAssetsData.PackagePath)
						);
						return;
					}
				}

				// We only have one object, so TakeID lookup may be unnecessary
				TArray<FCreateAssetsData> DataObjects;
				DataObjects.Add(CreateAssetsData);

				TOptional<FText> AssetCreationErrorMessage;
				FIngestAssetCreator::FPerTakeCallback Callback = FIngestAssetCreator::FPerTakeCallback(
					FIngestAssetCreator::FPerTakeCallback::Type::CreateLambda([&AssetCreationErrorMessage](TPair<int32, FIngestAssetCreator::FAssetCreationResult> InResult)
						{
							if (InResult.Value.HasError())
							{
								FAssetCreationError Error = InResult.Value.StealError();
								if (Error.GetError() != EAssetCreationError::Warning)
								{
									AssetCreationErrorMessage = Error.GetMessage();
								}
							}
						}), EDelegateExecutionThread::InternalThread);

				FIngestAssetCreator IngestAssetCreator;
				TArray<FCaptureDataAssetInfo> CaptureDataAssetInfos = IngestAssetCreator.CreateAssets_GameThread(DataObjects, MoveTemp(Callback));

				if (CaptureDataAssetInfos.IsEmpty())
				{
					OutErrorMessage = AssetCreationErrorMessage.Get(
						LOCTEXT("AssetCreationFailed", "Failed to create capture data asset"));
					return;
				}

				{
					const FString AssetPath = CreateAssetsData.PackagePath;
					FCaptureDataTakeInfo TakeInfo = BuildTakeInfo(CreateAssetsData, IngestCaptureData);
					FootageCaptureData = CreateFootageCaptureDataAsset_GameThread(AssetPath, CaptureDataAssetInfos[0], TakeInfo, bAutoSaveAssets);
				}
			}
		);
	}
	else
	{
		OutErrorMessage = IngestCaptureDataParseResult.GetError();
	}

	return FootageCaptureData;
}

} // namespace UE::CaptureManager

#undef LOCTEXT_NAMESPACE
