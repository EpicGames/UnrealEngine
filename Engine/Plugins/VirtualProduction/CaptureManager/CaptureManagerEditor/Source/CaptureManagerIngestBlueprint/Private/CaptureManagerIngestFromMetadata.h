// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/StopToken.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"

class FTakeMetadata;
struct FCaptureManagerConversionParams;
class UFootageCaptureData;

namespace UE::CaptureManager
{

/**
 * Configuration for IngestFromMetadata sourced from editor settings by the caller.
 * Settings are read once at the call site (per-device Blueprint function) and passed in
 * rather than accessed internally.
 */
struct FIngestPipelineOptions
{
	FString WorkingDirectory;
	TOptional<FString> EncoderPath;          // set only when third-party encoder is enabled
	FString CustomAudioCommandArguments;     // empty = use built-in default
	FString CustomVideoCommandArguments;     // empty = use built-in default
	bool bAutoSaveAssets = true;
};

/**
 * Shared ingest pipeline: working directory creation, conversion, intermediate-file parsing,
 * and game-thread asset creation. Called by every per-device Blueprint ingest function after
 * metadata extraction.
 *
 * @param InMetadata             Extracted take metadata (device-agnostic).
 * @param InTakeOriginDirectory  Absolute path to the directory that contains the source media files.
 * @param InParams               Conversion output format options.
 * @param InOptions              Pipeline configuration sourced from editor settings by the caller.
 * @param OutErrorMessage        Populated on failure.
 * @return                       The created asset, or nullptr on failure.
 */
UFootageCaptureData* IngestFromMetadata(
	const FTakeMetadata& InMetadata,
	const FString& InTakeOriginDirectory,
	const FCaptureManagerConversionParams& InParams,
	const FIngestPipelineOptions& InOptions,
	FText& OutErrorMessage,
	TOptional<FStopToken> InStopToken = {}
);

} // namespace UE::CaptureManager
