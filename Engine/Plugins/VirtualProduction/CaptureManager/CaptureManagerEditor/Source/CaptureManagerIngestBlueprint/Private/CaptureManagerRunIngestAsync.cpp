// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerRunIngestAsync.h"
#include "CaptureManagerIngestBlueprintModule.h"
#include "CaptureManagerIngestDispatcher.h"

#include "Modules/ModuleManager.h"

namespace UE::CaptureManager
{

int32 RunIngestAsync(
	ECaptureManagerIngestType IngestType,
	TFunction<UFootageCaptureData*(FText&, int32, const FStopToken&)> SyncIngestFn,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
)
{
	FCaptureManagerIngestBlueprintModule& Module = FModuleManager::GetModuleChecked<FCaptureManagerIngestBlueprintModule>("CaptureManagerIngestBlueprint");
	return Module.GetDispatcher().Enqueue(
		IngestType,
		MoveTemp(SyncIngestFn),
		MoveTemp(OnSuccess),
		MoveTemp(OnFailure)
	);
}

bool CancelIngest(int32 InIngestId)
{
	FCaptureManagerIngestBlueprintModule& Module = FModuleManager::GetModuleChecked<FCaptureManagerIngestBlueprintModule>("CaptureManagerIngestBlueprint");
	return Module.GetDispatcher().CancelIngest(InIngestId);
}

} // namespace UE::CaptureManager
