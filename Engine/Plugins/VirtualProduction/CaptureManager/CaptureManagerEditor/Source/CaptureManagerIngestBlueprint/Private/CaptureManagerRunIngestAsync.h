// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/StopToken.h"
#include "CaptureManagerIngestBlueprintLibrary.h"
#include "Templates/Function.h"

class UFootageCaptureData;

namespace UE::CaptureManager
{

/**
 * Generic async wrapper for all per-device Blueprint ingest functions.
 * Enqueues SyncIngestFn via the ingest dispatcher and marshals the result back to the
 * game thread, firing OnSuccess or OnFailure accordingly.
 */
int32 RunIngestAsync(
	ECaptureManagerIngestType IngestType,
	TFunction<UFootageCaptureData*(FText&, int32, const FStopToken&)> SyncIngestFn,
	FCaptureManagerIngestSuccess OnSuccess,
	FCaptureManagerIngestFailed OnFailure
);

/**
 * Cancel a queued or running ingest by its ID.
 * @return true if the IngestId was found and cancellation was initiated.
 */
bool CancelIngest(int32 InIngestId);

} // namespace UE::CaptureManager
