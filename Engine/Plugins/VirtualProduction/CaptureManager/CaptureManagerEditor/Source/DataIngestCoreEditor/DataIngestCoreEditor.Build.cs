// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DataIngestCoreEditor : ModuleRules
{
	public DataIngestCoreEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"Engine",
			"CaptureUtils",
			"CaptureDataCore"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"UnrealEd",
			"Media",
			"ImgMedia",
			"CameraCalibrationCore",
			"MeshDescription",
			"StaticMeshDescription",
			"AssetDefinition",
			"Json",
			"JsonUtilities",
			"SlateCore",
			"Slate",
			"InputCore",
			"PropertyEditor",
			"DataIngestCore",
			"CaptureDataEditor",
			"CaptureDataCore",
			"CaptureManagerEditorSettings",
			"NamingTokens"
		});

		ShortName = "DataIngCoEd";
	}
}
