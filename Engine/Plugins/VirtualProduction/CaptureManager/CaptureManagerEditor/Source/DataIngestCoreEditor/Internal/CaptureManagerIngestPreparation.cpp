// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerIngestPreparation.h"

#include "IngestAssetCreator.h"

#include "Asset/CaptureAssetSanitization.h"
#include "Settings/CaptureManagerEditorSettings.h"
#include "Settings/CaptureManagerEditorTemplateTokens.h"
#include "Utils/ParseTakeUtils.h"
#include "Utils/UnrealCalibrationParser.h"
#include "CameraCalibration.h"

#include "Engine/Engine.h"
#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "Subsystems/EditorAssetSubsystem.h"

namespace UE::CaptureManager::Private
{
UNamingTokensEngineSubsystem* GetTokenSubsystem()
{
	return GEngine ? GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>() : nullptr;
}

FNamingTokenFilterArgs MakeBaseTokenArgs(const UCaptureManagerEditorSettings* InSettings)
{
	FNamingTokenFilterArgs Args;
	if (const UCaptureManagerIngestNamingTokens* Tokens = InSettings->GetGeneralNamingTokens())
	{
		Args.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
	}
	return Args;
}
} // namespace UE::CaptureManager::Private

namespace UE::CaptureManager
{

FAssetNamingStrategy::FAssetNamingStrategy(const FIngestCaptureData& InIngestCaptureData) :
	FAssetNamingStrategy(InIngestCaptureData, TEXT(""), TEXT(""))
{
}

FAssetNamingStrategy::FAssetNamingStrategy(const FIngestCaptureData& InIngestCaptureData, FString InImportID, FString InDeviceName)
{
	check(IsInGameThread());

	if (const UCaptureManagerEditorSettings* Settings = GetDefault<UCaptureManagerEditorSettings>())
	{
		if (const UCaptureManagerIngestNamingTokens* Tokens = Settings->GetGeneralNamingTokens())
		{
			ImportNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::IdKey)).Name, InImportID);
			ImportNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::DeviceKey)).Name, InDeviceName);
			ImportNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::SlateKey)).Name, InIngestCaptureData.Slate);
			ImportNamedArgs.Add(Tokens->GetToken(FString(GeneralTokens::TakeKey)).Name, FString::FromInt(InIngestCaptureData.TakeNumber));
		}
	}
}

FString FAssetNamingStrategy::ResolveTemplate(
	const FString& InTemplate,
	const FNamingTokenFilterArgs& InTokenArgs,
	TConstArrayView<const FStringFormatNamedArguments*> InFormatArgSets,
	FSanitizeFunc InSanitize) const
{
	check(IsInGameThread());

	UNamingTokensEngineSubsystem* NamingTokensSubsystem = Private::GetTokenSubsystem();
	if (!NamingTokensSubsystem)
	{
		return {};
	}

	FString Result = InTemplate;
	for (const FStringFormatNamedArguments* ArgSet : InFormatArgSets)
	{
		Result = FString::Format(*Result, *ArgSet);
	}

	FNamingTokenResultData TokenResult = NamingTokensSubsystem->EvaluateTokenString(Result, InTokenArgs);
	Result = TokenResult.EvaluatedText.ToString();
	InSanitize(Result, TEXT('_'));

	return Result;
}

FStringFormatNamedArguments FAssetNamingStrategy::BuildVideoNamedArgs(
	const UCaptureManagerEditorSettings* InSettings,
	const FIngestCaptureData::FVideo& InVideo,
	const FFrameRate& InFrameRate)
{
	FStringFormatNamedArguments Args;
	if (const UCaptureManagerVideoNamingTokens* Tokens = InSettings->GetVideoNamingTokens())
	{
		Args.Add(Tokens->GetToken(FString(VideoTokens::NameKey)).Name, InVideo.Name);
		Args.Add(Tokens->GetToken(FString(VideoTokens::FrameRateKey)).Name, FString::Printf(TEXT("%.2f"), InFrameRate.AsDecimal()));
	}
	return Args;
}

FString FAssetNamingStrategy::GetImportFolder() const
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	if (!Settings)
	{
		return {};
	}

	FNamingTokenFilterArgs TokenArgs = Private::MakeBaseTokenArgs(Settings);

	const FStringFormatNamedArguments* ArgSets[] = { &ImportNamedArgs };
	return ResolveTemplate(Settings->GetVerifiedImportDirectory(), TokenArgs, ArgSets, &SanitizePackagePath);
}

FString FAssetNamingStrategy::GetCaptureDataAssetName() const
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	if (!Settings)
	{
		return {};
	}

	FNamingTokenFilterArgs TokenArgs = Private::MakeBaseTokenArgs(Settings);

	const FStringFormatNamedArguments* ArgSets[] = { &ImportNamedArgs };
	return ResolveTemplate(Settings->CaptureDataAssetName, TokenArgs, ArgSets, &SanitizeAssetName);
}

FString FAssetNamingStrategy::GetImageSequenceAssetName(const FIngestCaptureData::FVideo& InVideo, const FFrameRate& InFrameRate) const
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	if (!Settings)
	{
		return {};
	}

	FNamingTokenFilterArgs TokenArgs = Private::MakeBaseTokenArgs(Settings);
	if (const TObjectPtr<const UCaptureManagerVideoNamingTokens> Tokens = Settings->GetVideoNamingTokens())
	{
		TokenArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
	}

	FStringFormatNamedArguments VideoArgs = BuildVideoNamedArgs(Settings, InVideo, InFrameRate);

	const FStringFormatNamedArguments* ArgSets[] = { &VideoArgs, &ImportNamedArgs };
	return ResolveTemplate(Settings->ImageSequenceAssetName, TokenArgs, ArgSets, &SanitizeAssetName);
}

FString FAssetNamingStrategy::GetDepthSequenceAssetName(const FIngestCaptureData::FVideo& InVideo, const FFrameRate& InFrameRate) const
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	if (!Settings)
	{
		return {};
	}

	FNamingTokenFilterArgs TokenArgs = Private::MakeBaseTokenArgs(Settings);
	if (const TObjectPtr<const UCaptureManagerVideoNamingTokens> Tokens = Settings->GetVideoNamingTokens())
	{
		TokenArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
	}

	FStringFormatNamedArguments DepthArgs = BuildVideoNamedArgs(Settings, InVideo, InFrameRate);

	const FStringFormatNamedArguments* ArgSets[] = { &DepthArgs, &ImportNamedArgs };
	return ResolveTemplate(Settings->DepthSequenceAssetName, TokenArgs, ArgSets, &SanitizeAssetName);
}

FString FAssetNamingStrategy::GetAudioAssetName(const FIngestCaptureData::FAudio& InAudio) const
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	if (!Settings)
	{
		return {};
	}

	FNamingTokenFilterArgs TokenArgs = Private::MakeBaseTokenArgs(Settings);
	FStringFormatNamedArguments AudioArgs;
	if (const UCaptureManagerAudioNamingTokens* Tokens = Settings->GetAudioNamingTokens())
	{
		TokenArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
		AudioArgs.Add(Tokens->GetToken(FString(AudioTokens::NameKey)).Name, InAudio.Name);
	}

	const FStringFormatNamedArguments* ArgSets[] = { &AudioArgs, &ImportNamedArgs };
	return ResolveTemplate(Settings->SoundwaveAssetName, TokenArgs, ArgSets, &SanitizeAssetName);
}

FString FAssetNamingStrategy::GetCalibrationAssetName(const FIngestCaptureData::FCalibration& InCalibration) const
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	if (!Settings)
	{
		return {};
	}

	FNamingTokenFilterArgs TokenArgs = Private::MakeBaseTokenArgs(Settings);
	FStringFormatNamedArguments CalibArgs;
	if (const UCaptureManagerCalibrationNamingTokens* Tokens = Settings->GetCalibrationNamingTokens())
	{
		TokenArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
		CalibArgs.Add(Tokens->GetToken(FString(CalibTokens::NameKey)).Name, InCalibration.Name);
	}

	const FStringFormatNamedArguments* ArgSets[] = { &CalibArgs, &ImportNamedArgs };
	return ResolveTemplate(Settings->CalibrationAssetName, TokenArgs, ArgSets, &SanitizeAssetName);
}

FString FAssetNamingStrategy::GetLensFileAssetName(const FIngestCaptureData::FCalibration& InCalibration, const FCameraCalibration& InCameraCalibration) const
{
	UCaptureManagerEditorSettings* Settings = GetMutableDefault<UCaptureManagerEditorSettings>();
	if (!Settings)
	{
		return {};
	}

	FNamingTokenFilterArgs TokenArgs = Private::MakeBaseTokenArgs(Settings);
	FStringFormatNamedArguments CalibArgs;
	if (const UCaptureManagerCalibrationNamingTokens* CalibNamingTokens = Settings->GetCalibrationNamingTokens())
	{
		TokenArgs.AdditionalNamespacesToInclude.Add(CalibNamingTokens->GetNamespace());
		CalibArgs.Add(CalibNamingTokens->GetToken(FString(CalibTokens::NameKey)).Name, InCalibration.Name);
	}

	FStringFormatNamedArguments LensFileArgs;
	if (const UCaptureManagerLensFileNamingTokens* LensTokens = Settings->GetLensFileNamingTokens())
	{
		LensFileArgs.Add(LensTokens->GetToken(FString(LensFileTokens::CameraNameKey)).Name, InCameraCalibration.CameraId);
	}

	const FStringFormatNamedArguments* ArgSets[] = { &LensFileArgs, &CalibArgs, &ImportNamedArgs };
	return ResolveTemplate(Settings->LensFileAssetName, TokenArgs, ArgSets, &SanitizeAssetName);
}

FCreateAssetsData BuildAssetData(const FIngestCaptureData& InIngestCaptureData, const FAssetNamingStrategy& InNamingStrategy)
{
	check(IsInGameThread());

	FCreateAssetsData CreateAssetData;

	CreateAssetData.PackagePath = InNamingStrategy.GetImportFolder();
	CreateAssetData.CaptureDataAssetName = InNamingStrategy.GetCaptureDataAssetName();

	for (const FIngestCaptureData::FVideo& Video : InIngestCaptureData.Video)
	{
		const FFrameRate FrameRate = Video.FrameRate.IsSet() ? ParseFrameRate(Video.FrameRate.GetValue()) : FFrameRate();

		FCreateAssetsData::FImageSequenceData ImageSequenceData;
		ImageSequenceData.AssetName = InNamingStrategy.GetImageSequenceAssetName(Video, FrameRate);
		ImageSequenceData.FrameRate = FrameRate;
		ImageSequenceData.Name = Video.Name;
		ImageSequenceData.SequenceDirectory = Video.Path;
		ImageSequenceData.bTimecodePresent = Video.TimecodeStart.IsSet();
		ImageSequenceData.Timecode = Video.TimecodeStart.IsSet() ? ParseTimecode(Video.TimecodeStart.GetValue()) : FTimecode();
		ImageSequenceData.TimecodeRate = ImageSequenceData.FrameRate;

		CreateAssetData.ImageSequences.Add(ImageSequenceData);
	}

	for (const FIngestCaptureData::FVideo& Depth : InIngestCaptureData.Depth)
	{
		const FFrameRate FrameRate = Depth.FrameRate.IsSet() ? ParseFrameRate(Depth.FrameRate.GetValue()) : FFrameRate();

		FCreateAssetsData::FImageSequenceData DepthSequenceData;
		DepthSequenceData.AssetName = InNamingStrategy.GetDepthSequenceAssetName(Depth, FrameRate);
		DepthSequenceData.FrameRate = FrameRate;
		DepthSequenceData.Name = Depth.Name;
		DepthSequenceData.SequenceDirectory = Depth.Path;
		DepthSequenceData.bTimecodePresent = Depth.TimecodeStart.IsSet();
		DepthSequenceData.Timecode = Depth.TimecodeStart.IsSet() ? ParseTimecode(Depth.TimecodeStart.GetValue()) : FTimecode();
		DepthSequenceData.TimecodeRate = DepthSequenceData.FrameRate;
		CreateAssetData.DepthSequences.Add(DepthSequenceData);
	}

	for (const FIngestCaptureData::FAudio& Audio : InIngestCaptureData.Audio)
	{
		FCreateAssetsData::FAudioData AudioData;
		AudioData.AssetName = InNamingStrategy.GetAudioAssetName(Audio);
		AudioData.Name = Audio.Name;
		AudioData.WAVFile = Audio.Path;
		AudioData.bTimecodePresent = Audio.TimecodeStart.IsSet();
		AudioData.Timecode = Audio.TimecodeStart.IsSet() ? ParseTimecode(Audio.TimecodeStart.GetValue()) : FTimecode();
		AudioData.TimecodeRate = Audio.TimecodeRate.IsSet() ? ParseFrameRate(Audio.TimecodeRate.GetValue()) : FFrameRate();

		CreateAssetData.AudioClips.Add(AudioData);
	}

	for (const FIngestCaptureData::FCalibration& Calibration : InIngestCaptureData.Calibration)
	{
		FCreateAssetsData::FCalibrationData CalibrationData;
		CalibrationData.AssetName = InNamingStrategy.GetCalibrationAssetName(Calibration);

		FUnrealCalibrationParser::FParseResult Result = FUnrealCalibrationParser::Parse(Calibration.Path);

		if (Result.HasValue())
		{
			CalibrationData.CameraCalibrations = Result.StealValue();

			for (const FCameraCalibration& CamCalib : CalibrationData.CameraCalibrations)
			{
				CalibrationData.LensFileAssetNames.Add(CamCalib.CameraId, InNamingStrategy.GetLensFileAssetName(Calibration, CamCalib));
			}
		}

		CreateAssetData.Calibrations.Add(MoveTemp(CalibrationData));
	}

	if (!InIngestCaptureData.Video.IsEmpty())
	{
		TArray<FFrameNumber> DroppedFrameNumbers;
		DroppedFrameNumbers.Reserve(InIngestCaptureData.Video[0].DroppedFrames.Num());
		for (const uint32 DroppedFrameIndex : InIngestCaptureData.Video[0].DroppedFrames)
		{
			DroppedFrameNumbers.Add(FFrameNumber(static_cast<int32>(DroppedFrameIndex)));
		}
		CreateAssetData.CaptureExcludedFrames = PackIntoFrameRanges(MoveTemp(DroppedFrameNumbers));
	}

	return CreateAssetData;
}

FCaptureDataTakeInfo BuildTakeInfo(const FCreateAssetsData& InAssetsData, const FIngestCaptureData& InIngestCaptureData)
{
	FCaptureDataTakeInfo TakeInfo;
	TakeInfo.Name = InAssetsData.CaptureDataAssetName;
	TakeInfo.DeviceModel = InIngestCaptureData.DeviceModel;

	if (!InIngestCaptureData.Video.IsEmpty())
	{
		const FIngestCaptureData::FVideo& FirstVideo = InIngestCaptureData.Video[0];

		if (FirstVideo.FrameRate.IsSet())
		{
			TakeInfo.FrameRate = FirstVideo.FrameRate.GetValue();
		}
	}

	return TakeInfo;
}

UFootageCaptureData* CreateFootageCaptureDataAsset_GameThread(const FString& InAssetPath, const FCaptureDataAssetInfo& InResult, const FCaptureDataTakeInfo& InTakeInfo, bool bShouldSave)
{
	check(IsInGameThread());

	FString CaptureDataName = InTakeInfo.Name;
	CaptureDataName.TrimStartAndEndInline();
	CaptureDataName.ReplaceCharInline(' ', '_');

	if (FIngestAssetCreator::GetAssetIfExists<UFootageCaptureData>(InAssetPath, CaptureDataName))
	{
		return nullptr;
	}

	UFootageCaptureData* CaptureData = FIngestAssetCreator::CreateAsset<UFootageCaptureData>(InAssetPath, CaptureDataName);
	if (CaptureData)
	{
		CaptureData->ImageSequences.Reset();
		CaptureData->DepthSequences.Reset();
		CaptureData->CameraCalibrations.Reset();
		CaptureData->AudioTracks.Reset();

		for (const FCaptureDataAssetInfo::FImageSequence& ImageSequence : InResult.ImageSequences)
		{
			CaptureData->ImageSequences.Add(ImageSequence.Asset);
		}

		for (const FCaptureDataAssetInfo::FImageSequence& DepthSequence : InResult.DepthSequences)
		{
			CaptureData->DepthSequences.Add(DepthSequence.Asset);
		}

		for (const FCaptureDataAssetInfo::FAudio& Audio : InResult.Audios)
		{
			CaptureData->AudioTracks.Add(Audio.Asset);
		}

		for (const FCaptureDataAssetInfo::FCalibration& Calibration : InResult.Calibrations)
		{
			CaptureData->CameraCalibrations.Add(Calibration.Asset);
		}

		CaptureData->Metadata.FrameRate = InTakeInfo.FrameRate;
		CaptureData->Metadata.DeviceModelName = InTakeInfo.DeviceModel;
		CaptureData->Metadata.SetDeviceClass(InTakeInfo.DeviceModel);
		CaptureData->CaptureExcludedFrames = InResult.CaptureExcludedFrames;

		if (bShouldSave)
		{
			IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

			TArray<FAssetData> AssetsData;
			AssetRegistry.GetAssetsByPath(FName{ *InAssetPath }, AssetsData, true, false);

			TArray<UPackage*> Packages;
			for (const FAssetData& AssetData : AssetsData)
			{
				if (UObject* Asset = AssetData.GetAsset())
				{
					Packages.AddUnique(Asset->GetPackage());
				}
			}

			if (!Packages.IsEmpty())
			{
				UEditorLoadingAndSavingUtils::SavePackages(Packages, true);
			}
		}
	}

	return CaptureData;
}

}
