// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IngestAssetCreator.h"
#include "IngestCaptureData.h"

#include "Misc/FrameRate.h"

#define UE_API DATAINGESTCOREEDITOR_API

struct FCameraCalibration;
struct FNamingTokenFilterArgs;
class UCaptureManagerEditorSettings;

namespace UE::CaptureManager
{

struct FCaptureDataTakeInfo
{
	FString Name;
	double FrameRate = 0.0;
	FString DeviceModel;
};

class FAssetNamingStrategy
{
public:
	UE_API FAssetNamingStrategy(const FIngestCaptureData& InIngestCaptureData);
	UE_API FAssetNamingStrategy(const FIngestCaptureData& InIngestCaptureData, FString InImportID, FString InDeviceName);

	UE_API FString GetImportFolder() const;
	UE_API FString GetCaptureDataAssetName() const;
	UE_API FString GetImageSequenceAssetName(const FIngestCaptureData::FVideo& InVideo, const FFrameRate& InFrameRate) const;
	UE_API FString GetDepthSequenceAssetName(const FIngestCaptureData::FVideo& InVideo, const FFrameRate& InFrameRate) const;
	UE_API FString GetAudioAssetName(const FIngestCaptureData::FAudio& InAudio) const;
	UE_API FString GetCalibrationAssetName(const FIngestCaptureData::FCalibration& InCalibration) const;
	UE_API FString GetLensFileAssetName(const FIngestCaptureData::FCalibration& InCalibration, const FCameraCalibration& InCameraCalibration) const;

private:
	using FSanitizeFunc = void(*)(FString&, FString::ElementType);

	FString ResolveTemplate(
		const FString& InTemplate,
		const FNamingTokenFilterArgs& InTokenArgs,
		TConstArrayView<const FStringFormatNamedArguments*> InFormatArgSets,
		FSanitizeFunc InSanitize) const;

	static FStringFormatNamedArguments BuildVideoNamedArgs(
		const UCaptureManagerEditorSettings* InSettings,
		const FIngestCaptureData::FVideo& InVideo,
		const FFrameRate& InFrameRate);

	FStringFormatNamedArguments ImportNamedArgs;
};

UE_API FCreateAssetsData BuildAssetData(const FIngestCaptureData& InIngestCaptureData, const FAssetNamingStrategy& InNamingStrategy);

/** Populates a FCaptureDataTakeInfo from the asset name, device model, and first video entry of the ingest data. */
UE_API FCaptureDataTakeInfo BuildTakeInfo(const FCreateAssetsData& InAssetsData, const FIngestCaptureData& InIngestCaptureData);

/** Creates a UFootageCaptureData asset from prepared asset info and take metadata. Returns nullptr if the asset already exists. */
UE_API UFootageCaptureData* CreateFootageCaptureDataAsset_GameThread(const FString& InAssetPath, const FCaptureDataAssetInfo& InResult, const FCaptureDataTakeInfo& InTakeInfo, bool bShouldSave = true);

}

#undef UE_API
