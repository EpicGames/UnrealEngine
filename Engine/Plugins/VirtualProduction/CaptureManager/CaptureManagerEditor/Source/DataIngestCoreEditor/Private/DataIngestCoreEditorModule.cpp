// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FDataIngestCoreEditorModule : public IModuleInterface
{
public:
	
	void StartupModule()
	{
	}

	void ShutdownModule()
	{
	}
};

IMPLEMENT_MODULE(FDataIngestCoreEditorModule, DataIngestCoreEditor)
