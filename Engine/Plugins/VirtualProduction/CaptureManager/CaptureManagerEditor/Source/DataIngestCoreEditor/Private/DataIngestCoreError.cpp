// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataIngestCoreError.h"

FAssetCreationError::FAssetCreationError(FText InMessage, EAssetCreationError InError)
	: Message(MoveTemp(InMessage))
	, Error(InError)
{
}

const FText& FAssetCreationError::GetMessage() const
{
	return Message;
}

EAssetCreationError FAssetCreationError::GetError() const
{
	return Error;
}