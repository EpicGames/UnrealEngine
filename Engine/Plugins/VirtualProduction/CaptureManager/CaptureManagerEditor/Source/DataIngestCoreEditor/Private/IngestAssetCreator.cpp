// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestAssetCreator.h"

#include "Misc/ScopedSlowTask.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "ImgMediaSource.h"
#include "Sound/SoundWave.h"
#include "AssetImportTask.h"
#include "ObjectTools.h"
#include "ImageSequenceUtils.h"

#include "Editor.h"
#include "Subsystems/EditorAssetSubsystem.h"

#include "Utils/ParseTakeUtils.h"

#include "Logging/LogMacros.h"

#define LOCTEXT_NAMESPACE "IngestAssetCreator"

DEFINE_LOG_CATEGORY_STATIC(LogIngestAssetCreator, Log, All);

namespace UE::CaptureManager {

TArray<FCaptureDataAssetInfo> FIngestAssetCreator::CreateAssets_GameThread(const TArray<FCreateAssetsData>& InOutCreateAssetDataList,
																		   FPerTakeCallback InPerTakeCallback)
{
	TArray<int32> TakesToRemove;

	auto PerTakesLambda = FPerTakeCallback([PerTakeCallback = MoveTemp(InPerTakeCallback), &TakesToRemove](FPerTakeResult InResult) mutable
	{
		if (InResult.Value.HasError() && InResult.Value.GetError().GetError() != EAssetCreationError::Warning)
		{
			TakesToRemove.Add(InResult.Key);
		}

		PerTakeCallback(MoveTemp(InResult));
	}, EDelegateExecutionThread::InternalThread);

	TArray<FCaptureDataAssetInfo> Takes;
	CreateTakeAssets_GameThread(InOutCreateAssetDataList, PerTakesLambda, Takes);
	RemoveTakes(MoveTemp(TakesToRemove), Takes);

	VerifyIngestedData_GameThread(InOutCreateAssetDataList, Takes, PerTakesLambda);
	RemoveTakes(MoveTemp(TakesToRemove), Takes);

	return Takes;
}

void FIngestAssetCreator::CreateTakeAssets_GameThread(const TArray<FCreateAssetsData>& InOutCreateAssetsData,
													  const FPerTakeCallback& InPerTakeCallback,
													  TArray<FCaptureDataAssetInfo>& OutTakes)
{
	struct FTakeAssetImportTaskInfo
	{
		// The index in the current OutTakes array that this import task info refers to
		int32 TakeEntryIndex = INDEX_NONE;

		// The index of the audio clip to import for the given FMetaHumanTake
		int32 AudioClipIndex = INDEX_NONE;
	};

	FScopedSlowTask CreateAssetsProgress(InOutCreateAssetsData.Num(), LOCTEXT("CreateAssetsTask", "Creating assets for ingested takes"));
	CreateAssetsProgress.MakeDialog();

	// Strong ptrs to prevent the import tasks being garbage collected until the end of this function
	TMap<TStrongObjectPtr<UAssetImportTask>, FTakeAssetImportTaskInfo> ImportTasksMap;

	for (int32 Index = 0; Index < InOutCreateAssetsData.Num(); ++Index)
	{
		const FCreateAssetsData& CreateAssetData = InOutCreateAssetsData[Index];

		FCaptureDataAssetInfo& Take = OutTakes.AddDefaulted_GetRef();
		Take.TakeId = CreateAssetData.TakeId;

		CreateAssetsProgress.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("CreateAssetsForTakeMessage", "Creating assets ({0} of {1}) for {2}"), Index + 1, InOutCreateAssetsData.Num(), FText::FromString(CreateAssetData.PackagePath)));

		// Create the image sequence assets
		TValueOrError<void, FText> CreateTakeAssetsResult = CreateTakeAssetViews_GameThread(CreateAssetData, Take);
		if (CreateTakeAssetsResult.HasError())
		{
			FPerTakeResult TakeResult = FPerTakeResult(CreateAssetData.TakeId, MakeError(FAssetCreationError(CreateTakeAssetsResult.StealError())));
			InPerTakeCallback(MoveTemp(TakeResult));

			continue;
		}

		for (int32 AudioClipIndex = 0; AudioClipIndex < CreateAssetData.AudioClips.Num(); ++AudioClipIndex)
		{
			const FCreateAssetsData::FAudioData& Audio = CreateAssetData.AudioClips[AudioClipIndex];

			TObjectPtr<USoundWave> AudioAsset = GetAssetIfExists<USoundWave>(CreateAssetData.PackagePath, Audio.AssetName);
			
			if (AudioAsset == nullptr)
			{
				// If audio is disabled, USoundWave asset cannot be created as the required decoder will not be available
				if (FApp::CanEverRenderAudio())
				{
					// Need to import it from the WAVFilePath
					TStrongObjectPtr<UAssetImportTask> ImportTask(NewObject<UAssetImportTask>());
					ImportTask->bAutomated = true;
					ImportTask->bReplaceExisting = true;
					ImportTask->bSave = false;
					ImportTask->DestinationPath = CreateAssetData.PackagePath;
					ImportTask->DestinationName = Audio.AssetName;
					ImportTask->Filename = Audio.WAVFile;

					FTakeAssetImportTaskInfo ImportTaskInfo;
					ImportTaskInfo.TakeEntryIndex = Index;
					ImportTaskInfo.AudioClipIndex = AudioClipIndex;
					ImportTasksMap.Emplace(MoveTemp(ImportTask), MoveTemp(ImportTaskInfo));
				}
				else
				{
					FText ErrorText = FText::Format(LOCTEXT("IngestAssetCreatorError_AudioDisabled", "Failed to create USoundWave '{0}' at package path '{1}'.\nAudio is disabled in UE. If this is not intentional, please check command line arguments for \"-nosound\"."), FText::FromString(Audio.AssetName), FText::FromString(CreateAssetData.PackagePath));
					FPerTakeResult AudioResult = FPerTakeResult(CreateAssetData.TakeId, MakeError(FAssetCreationError(ErrorText)));
					InPerTakeCallback(MoveTemp(AudioResult));
				}
			}
			else
			{
				FText ErrorText = FText::Format(LOCTEXT("IngestAssetCreatorError_SoundWaveAlreadyExists", "Failed to create USoundWave '{0}'. Asset with name '{1}' already exists at package path '{2}'"), FText::FromString(Audio.AssetName), FText::FromString(Audio.AssetName), FText::FromString(CreateAssetData.PackagePath));
				FPerTakeResult AudioResult = FPerTakeResult(CreateAssetData.TakeId, MakeError(FAssetCreationError(ErrorText)));
				InPerTakeCallback(MoveTemp(AudioResult));
			}
		}

		for (const FCreateAssetsData::FCalibrationData& Calibration : CreateAssetData.Calibrations)
		{
			TObjectPtr<UCameraCalibration> CalibrationAsset = GetAssetIfExists<UCameraCalibration>(CreateAssetData.PackagePath, Calibration.AssetName);
			if (CalibrationAsset)
			{
				FText ErrorText = FText::Format(LOCTEXT("IngestAssetCreatorError_CamreaCalibrationAlreadyExists", "Failed to create UCameraCalibration '{0}'. Asset with name '{1}' already exists at package path '{2}'"), FText::FromString(Calibration.AssetName), FText::FromString(Calibration.AssetName), FText::FromString(CreateAssetData.PackagePath));
				FPerTakeResult CalibResult = FPerTakeResult(CreateAssetData.TakeId, MakeError(FAssetCreationError(ErrorText)));
				InPerTakeCallback(MoveTemp(CalibResult));
				continue;
			}

			CalibrationAsset = CreateAsset<UCameraCalibration>(CreateAssetData.PackagePath, Calibration.AssetName);
			CalibrationAsset->CameraCalibrations.Reset();
			CalibrationAsset->StereoPairs.Reset();

			TArray<FCameraCalibration> CameraCalibrations = Calibration.CameraCalibrations;
			CalibrationAsset->ConvertFromTrackerNodeCameraModels(CameraCalibrations, Calibration.LensFileAssetNames, true);

			FCaptureDataAssetInfo::FCalibration CalibrationAssetInfo;
			CalibrationAssetInfo.Asset = MoveTemp(CalibrationAsset);

			Take.Calibrations.Add(MoveTemp(CalibrationAssetInfo));
		}
			
		Take.CaptureExcludedFrames = CreateAssetData.CaptureExcludedFrames;
	}

	// Run all the import asset tasks in a bundle
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>(TEXT("AssetTools")).Get();

	{
		// Convert the strong object ptrs into an array of non-owning ptrs for the import call
		TArray<UAssetImportTask*> ImportTasks;
		ImportTasks.Reserve(ImportTasksMap.Num());

		for (const TPair<TStrongObjectPtr<UAssetImportTask>, FTakeAssetImportTaskInfo>& ImportTasksEntry : ImportTasksMap)
		{
			ImportTasks.Add(ImportTasksEntry.Key.Get());
		}

		AssetTools.ImportAssetTasks(ImportTasks);
	}

	// Map import results back to the takes
	for (int32 Index = 0; Index < OutTakes.Num(); ++Index)
	{
		TMap<UAssetImportTask*, FTakeAssetImportTaskInfo> TakeTasks;

		for (const TPair<TStrongObjectPtr<UAssetImportTask>, FTakeAssetImportTaskInfo>& ImportTasksEntry : ImportTasksMap)
		{
			if (ImportTasksEntry.Value.TakeEntryIndex == Index)
			{
				TakeTasks.Emplace(ImportTasksEntry.Key.Get(), ImportTasksEntry.Value);
			}
		}

		FCaptureDataAssetInfo& Take = OutTakes[Index];

		if (TakeTasks.IsEmpty())
		{
			FAssetCreationResult Result = CheckTakeAssets(Take);

			if (Result.HasError())
			{
				FPerTakeResult TakeResult = FPerTakeResult(Take.TakeId, MoveTemp(Result));
				InPerTakeCallback(MoveTemp(TakeResult));
			}

			continue;
		}

		TArray<FAssetCreationResult> TakeResults;
		TakeResults.Reserve(TakeTasks.Num());

		for (const TPair<UAssetImportTask*, FTakeAssetImportTaskInfo>& TakeImportTask : TakeTasks)
		{
			const FTakeAssetImportTaskInfo& ImportTaskInfo = TakeImportTask.Value;
			const TObjectPtr<UAssetImportTask> ImportTask = TakeImportTask.Key;
			const FCreateAssetsData& CreateAssetData = InOutCreateAssetsData[ImportTaskInfo.TakeEntryIndex];

			if (ImportTaskInfo.AudioClipIndex != INDEX_NONE)
			{
				const FCreateAssetsData::FAudioData& AudioClipData = CreateAssetData.AudioClips[ImportTaskInfo.AudioClipIndex];
				TakeResults.Add(AssignAudioAsset(AudioClipData, ImportTask, Take));

				FString AudioAssetName = AudioClipData.Name;
				if (USoundWave* Asset = GetAssetIfExists<USoundWave>(CreateAssetData.PackagePath, AudioAssetName); Asset != nullptr)
				{
					UE_LOGF(LogIngestAssetCreator, Display, "SoundWave asset created successfully");
				}
			}
		}

		for (FAssetCreationResult& Result : TakeResults)
		{
			if (Result.HasError())
			{
				FPerTakeResult TakeResult = FPerTakeResult(Take.TakeId, MoveTemp(Result));
				InPerTakeCallback(MoveTemp(TakeResult));

				break;
			}
		}
	}
}

TValueOrError<void, FText> FIngestAssetCreator::CreateTakeAssetViews_GameThread(const FCreateAssetsData& InCreateAssetData,
														  FCaptureDataAssetInfo& OutTake)
{
	TArray<FText> ErrorMessages;
	for (const FCreateAssetsData::FImageSequenceData& ImageSequence : InCreateAssetData.ImageSequences)
	{
		// TODO: Check relationship between PackagePath and Asset Name
		TValueOrError<TObjectPtr<UImgMediaSource>, FText> VideoAssetResult = CreateImageSequenceAsset(InCreateAssetData.PackagePath, ImageSequence);
		if (VideoAssetResult.HasError())
		{
			ErrorMessages.Add(VideoAssetResult.GetError());
			UE_LOGF(LogIngestAssetCreator, Error, "Failed to create UImgMediaSource asset for image sequence '%ls'", *(ImageSequence.Name));
			continue;
		}

		FCaptureDataAssetInfo::FImageSequence Sequence;
		Sequence.Asset = MoveTemp(VideoAssetResult.GetValue());
		Sequence.Timecode = ImageSequence.Timecode;
		Sequence.TimecodeRate = ImageSequence.TimecodeRate;

		OutTake.ImageSequences.Add(MoveTemp(Sequence));
	}

	for (const FCreateAssetsData::FImageSequenceData& DepthSequence : InCreateAssetData.DepthSequences)
	{
		TValueOrError<TObjectPtr<UImgMediaSource>, FText> DepthAssetResult = CreateImageSequenceAsset(InCreateAssetData.PackagePath, DepthSequence);
		if (DepthAssetResult.HasError())
		{
			ErrorMessages.Add(DepthAssetResult.GetError());
			UE_LOGF(LogIngestAssetCreator, Error, "Failed to create UImgMediaSource asset for depth sequence '%ls'", *(DepthSequence.Name));
			continue;
		}

		FCaptureDataAssetInfo::FImageSequence Sequence;
		Sequence.Asset = MoveTemp(DepthAssetResult.GetValue());
		Sequence.Timecode = DepthSequence.Timecode;
		Sequence.TimecodeRate = DepthSequence.TimecodeRate;

		OutTake.DepthSequences.Add(MoveTemp(Sequence));
	}

	if (!ErrorMessages.IsEmpty())
	{
		// Make combined error message
		FText CombinedErrors = FText::Join(FText::FromString(TEXT("\n")), ErrorMessages);
		return MakeError(MoveTemp(CombinedErrors));
	}

	return MakeValue();
}

TValueOrError<TObjectPtr<UImgMediaSource>, FText> FIngestAssetCreator::CreateImageSequenceAsset(const FString& InPackagePath, 
																		  const FCreateAssetsData::FImageSequenceData& InImageSequenceData)
{
	if (TObjectPtr<UImgMediaSource> ExistingImageSequenceAsset = GetAssetIfExists<UImgMediaSource>(InPackagePath, InImageSequenceData.AssetName))
	{	
		FText ErrorText = FText::Format(LOCTEXT("IngestAssetCreatorError_ImgMediaAlreadyExists", "Failed to create UImgMediaSource '{0}'. Asset with name '{1}' already exists at package path '{2}'"), FText::FromString(InImageSequenceData.AssetName), FText::FromString(InImageSequenceData.AssetName), FText::FromString(InPackagePath));		
		UE_LOGF(LogIngestAssetCreator, Error, "%ls", *ErrorText.ToString());
		return MakeError(MoveTemp(ErrorText));
	}
	
	TObjectPtr<UImgMediaSource> ImageSequenceAsset = CreateAsset<UImgMediaSource>(InPackagePath, InImageSequenceData.AssetName);
	if (!ImageSequenceAsset)
	{
		FText ErrorText = FText::Format(LOCTEXT("IngestAssetCreatorError_FailedToCreateImgMedia", "Failed to create UImgMediaSource '{0}'"), FText::FromString(InImageSequenceData.AssetName));
		UE_LOGF(LogIngestAssetCreator, Error, "%ls", *ErrorText.ToString());
		return MakeError(MoveTemp(ErrorText));
	}

	ImageSequenceAsset->SetTokenizedSequencePath(InImageSequenceData.SequenceDirectory);
	ImageSequenceAsset->FrameRateOverride = InImageSequenceData.FrameRate;
	ImageSequenceAsset->StartTimecode = InImageSequenceData.Timecode;

	// Add MetaData
	if (UPackage* AssetPackage = ImageSequenceAsset->GetPackage())
	{
		AssetPackage->GetMetaData().SetValue(ImageSequenceAsset, TEXT("CameraId"), *InImageSequenceData.Name);
	}

	return MakeValue(MoveTemp(ImageSequenceAsset));
}

FIngestAssetCreator::FAssetCreationResult FIngestAssetCreator::AssignAudioAsset(const FCreateAssetsData::FAudioData& AudioClip,
																				const TObjectPtr<UAssetImportTask>& InAssetImportTask, 
																				FCaptureDataAssetInfo& OutTake)
{
	if (InAssetImportTask->GetObjects().IsEmpty())
	{
		FText Message = LOCTEXT("IngestError_AudioImport", "Error importing audio clip");
		return MakeError(FAssetCreationError(MoveTemp(Message)));
	}

	FCaptureDataAssetInfo::FAudio Audio;

	Audio.Asset = Cast<USoundWave>(InAssetImportTask->GetObjects()[0]);

	if (ensure(IsValid(Audio.Asset)))
	{
		PrepareSoundWave(AudioClip, InAssetImportTask, Audio.Asset);
	}

	OutTake.Audios.Add(MoveTemp(Audio));

	return MakeValue();
}

FString FIngestAssetCreator::CreateTimecodeString(FTimecode InTimecode, FFrameRate InFrameRate)
{
	return FString::Printf(TEXT("%s %s @ %.2f fps"), *InTimecode.ToString(), InTimecode.bDropFrameFormat ? TEXT("DF") : TEXT("ND"), InFrameRate.AsDecimal());
}

FString FIngestAssetCreator::CreateAssetPathString(const UAssetImportTask* InAssetImportTask)
{
	return FString::Printf(TEXT("%s (Created from %s)"), *InAssetImportTask->DestinationPath, *InAssetImportTask->Filename);
}

void FIngestAssetCreator::StampWithTakeMetadataTimecode(const FCreateAssetsData::FAudioData& InAudioClip, const UAssetImportTask* InAssetImportTask, USoundWave* OutSoundWave)
{
	const uint32 SampleRate = OutSoundWave->GetSampleRateForCurrentPlatform();

	// The sample rate can be zero in a number of circumstances (GetSampleRateForCurrentPlatform is not a simple 
	// function)
	const bool bIsValidSampleRate = SampleRate > 0;
	check(bIsValidSampleRate);

	if (bIsValidSampleRate)
	{
		const bool bIsValidTimecodeRate = InAudioClip.TimecodeRate != FFrameRate();
		check(bIsValidTimecodeRate);

		if (bIsValidTimecodeRate)
		{
			const FTimespan AudioTimespan = InAudioClip.Timecode.ToTimespan(InAudioClip.TimecodeRate);

			FSoundWaveTimecodeInfo NewSoundWaveTimecodeInfo;
			NewSoundWaveTimecodeInfo.NumSamplesPerSecond = SampleRate;
			NewSoundWaveTimecodeInfo.TimecodeRate = InAudioClip.TimecodeRate;
			NewSoundWaveTimecodeInfo.NumSamplesSinceMidnight = AudioTimespan.GetTotalSeconds() * SampleRate;
			NewSoundWaveTimecodeInfo.bTimecodeIsDropFrame = InAudioClip.Timecode.bDropFrameFormat;

			OutSoundWave->SetTimecodeInfo(NewSoundWaveTimecodeInfo);
		}
		else
		{
			UE_LOGF(
				LogIngestAssetCreator,
				Error,
				"Audio timecode rate is invalid (%.2f): %ls",
				InAudioClip.TimecodeRate.AsDecimal(),
				*CreateAssetPathString(InAssetImportTask)
			);
		}
	}
	else
	{
		UE_LOGF(
			LogIngestAssetCreator,
			Error,
			"Audio sample rate is invalid (%d Hz): %ls",
			SampleRate,
			*CreateAssetPathString(InAssetImportTask)
		);
	}
}

bool FIngestAssetCreator::IsValidAudioTimecodeRate(FFrameRate InTimecodeRate, uint32 InNumSamplesPerSecond)
{
	// When the iXML chunk is missing from the wav file, the timecode rate gets set to the sample rate. 
	// See FWaveModInfo::ReadWaveInfo in Engine/Source/Runtime/Engine/Private/Audio.cpp for details.
	const bool bIsMissingXmlChunk = InTimecodeRate == FFrameRate(InNumSamplesPerSecond, 1);
	const bool bTimecodeRateIsDefaulted = InTimecodeRate == FFrameRate();

	if (bIsMissingXmlChunk)
	{
		UE_LOGF(LogIngestAssetCreator, Warning, "Timecode rate in the imported audio is %.2f fps, perhaps a missing iXML chunk?", InTimecodeRate.AsDecimal());
	}

	return !(bIsMissingXmlChunk || bTimecodeRateIsDefaulted);
}

void FIngestAssetCreator::PrepareSoundWave(const FCreateAssetsData::FAudioData& InAudioClip, const UAssetImportTask* InAssetImportTask, USoundWave* OutSoundWave)
{
	OutSoundWave->SetSoundAssetCompressionType(ESoundAssetCompressionType::BinkAudio);

	const TOptional<FSoundWaveTimecodeInfo> SoundWaveTimecodeInfo = OutSoundWave->GetTimecodeInfo();
	
	if (SoundWaveTimecodeInfo)
	{
		const bool bIsValidTimecodeRate = IsValidAudioTimecodeRate(SoundWaveTimecodeInfo->TimecodeRate, SoundWaveTimecodeInfo->NumSamplesPerSecond);
		constexpr bool bRollover = true;
		const FTimecode SoundWaveTimecode(SoundWaveTimecodeInfo->GetNumSecondsSinceMidnight(), SoundWaveTimecodeInfo->TimecodeRate, bRollover);

		// Report the invalid timecode rate before doing anything else - that way the user can understand why certain decisions are made
		if (!bIsValidTimecodeRate)
		{
			if (InAudioClip.bTimecodePresent)
			{
				UE_LOGF(
					LogIngestAssetCreator,
					Warning,
					"Timecode is present in the audio file, but we could not determine a valid timecode rate (%ls). Falling back to the value extracted from the take metadata (%ls): %ls",
					*CreateTimecodeString(SoundWaveTimecode, SoundWaveTimecodeInfo->TimecodeRate),
					*CreateTimecodeString(InAudioClip.Timecode, InAudioClip.TimecodeRate),
					*CreateAssetPathString(InAssetImportTask)
				);

				StampWithTakeMetadataTimecode(InAudioClip, InAssetImportTask, OutSoundWave);
			}
			else
			{
				UE_LOGF(
					LogIngestAssetCreator,
					Warning,
					"Timecode is present in the audio file, but we could not determine a valid timecode rate (%ls). This will need to be fixed manually",
					*CreateTimecodeString(SoundWaveTimecode, SoundWaveTimecodeInfo->TimecodeRate)
				);
			}
		}
		else
		{
			if (InAudioClip.bTimecodePresent)
			{
				// The user supplied a timecode in the take metadata, however the audio already has a valid timecode which we
				// don't want to overwrite, so we warn the user that we are going to ignore it (Audio/Video alignment should not
				// be achieved by altering the input data!)
				UE_LOGF(
					LogIngestAssetCreator,
					Warning,
					"Ignoring the timecode extracted from the take metadata (%ls), the audio already has timecode (%ls): %ls",
					*CreateTimecodeString(InAudioClip.Timecode, InAudioClip.TimecodeRate),
					*CreateTimecodeString(SoundWaveTimecode, SoundWaveTimecodeInfo->TimecodeRate),
					*CreateAssetPathString(InAssetImportTask)
				);
			}
		}
	}
	else if (InAudioClip.bTimecodePresent)
	{
		// The audio file did not contain timecode which made it into the sound wave during import, however the user
		// has supplied one in the take metadata, so we update the sound wave to use it.
		UE_LOGF(
			LogIngestAssetCreator,
			Display,
			"Imported audio does not have timecode, using the timecode extracted from the take metadata (%ls): %ls",
			*CreateTimecodeString(InAudioClip.Timecode, InAudioClip.TimecodeRate),
			*CreateAssetPathString(InAssetImportTask)
		);

		StampWithTakeMetadataTimecode(InAudioClip, InAssetImportTask, OutSoundWave);
	}

	if (!OutSoundWave->GetTimecodeInfo())
	{
		// Not an invalid state, but warn the user that their audio does not have timecode
		UE_LOGF(
			LogIngestAssetCreator,
			Warning,
			"No audio timecode in sound wave asset: %ls",
			*CreateAssetPathString(InAssetImportTask)
		);
	}
}

#define CHECK_UOBJECT_RET(Obj, Name) if (!Obj) { return MakeError(FAssetCreationError(FText::Format(LOCTEXT("CheckError_AssetNotFound", "Asset doesn't exist: {0}"), FText::FromString(Name)), EAssetCreationError::NotFound)); }
#define CHECK_RET(Condition, Message) if (!(Condition)) { return MakeError(FAssetCreationError(FText::Format(LOCTEXT("CheckError_IngestResultError", "Checking ingest result failed: {0}"), Message))); }
#define CHECK_UOBJECT_WARN(Obj, Name) if (!Obj) { UE_LOGF(LogIngestAssetCreator, Warning, "Asset doesn't exist: %ls", *Name); }
#define CHECK_WARN(Condition, Message) if (!(Condition)) { UE_LOGF(LogIngestAssetCreator, Warning, "Checking ingest result failed: %ls", *Message.ToString()); }

void FIngestAssetCreator::VerifyIngestedData_GameThread(const TArray<FCreateAssetsData>& InCreateAssetsData,
														const TArray<FCaptureDataAssetInfo>& InCreatedTakes,
														const FPerTakeCallback& InPerTakeCallback)
{
	IFileManager& FileManager = IFileManager::Get();
	for (const FCreateAssetsData& AssetsData : InCreateAssetsData)
	{
		const FCaptureDataAssetInfo* FoundTake = InCreatedTakes.FindByPredicate([TakeId = AssetsData.TakeId](const FCaptureDataAssetInfo& InElem)
		{
			return InElem.TakeId == TakeId;
		});

		if (!FoundTake)
		{
			continue;
		}

		FAssetCreationResult Result = CheckCreatedTakeAssets_GameThread(AssetsData);

		if (Result.HasError())
		{
			FText Message = LOCTEXT("IngestError_Validation_IngestedFiles", "Validation of ingested files failed");

			FPerTakeResult TakeResult = FPerTakeResult(AssetsData.TakeId, MakeError(FAssetCreationError(MoveTemp(Message))));
			InPerTakeCallback(MoveTemp(TakeResult));

			continue;
		}

		Result = CheckCreatedTakeStruct_GameThread(*FoundTake);

		if (Result.HasError())
		{
			FText Message = LOCTEXT("IngestError_Validation_TakeStructure", "Validation of created Take structure failed");

			FPerTakeResult TakeResult = FPerTakeResult(AssetsData.TakeId, MakeError(FAssetCreationError(MoveTemp(Message))));
			InPerTakeCallback(MoveTemp(TakeResult));

			continue;
		}
		
		// If we have both calibrations and image sequences we should check that the camera names in the calibration
		// correspond to camera names for the image sequences. This is not a firm requirement but the general expectation
		// is for the ingested calibrations to map to the ingested image sequences. If we do not have a corresponding image
		// sequence for each calibration, we warn the user.
		// 
		// Note: If we have additional image sequences that do not correspond to a calibration, this is not considered an error case
		if (!AssetsData.Calibrations.IsEmpty() && !AssetsData.ImageSequences.IsEmpty())
		{
			TArray<FString> ImageSequenceNames;
			for (const FCreateAssetsData::FImageSequenceData& ImageSequenceInfo : AssetsData.ImageSequences)
			{
				ImageSequenceNames.Emplace(ImageSequenceInfo.Name);
			}
			FString ImageSequenceNamesString = FString::Join(ImageSequenceNames, TEXT(", "));

			for (const FCreateAssetsData::FCalibrationData& CalibAssetData : AssetsData.Calibrations)
			{
				for (const FCameraCalibration& Calib : CalibAssetData.CameraCalibrations)
				{
					// Check if we have a corresponding image sequence
					bool bHasCorrespondence = AssetsData.ImageSequences.ContainsByPredicate([&](const FCreateAssetsData::FImageSequenceData& item)
						{
							return item.Name == Calib.CameraId;
						}
					);

					if (!bHasCorrespondence)
					{
						UE_LOGF(LogIngestAssetCreator, Warning, "The ingested Camera Calibration contains a CameraId \"%ls\" which does not correspond to any of the ingested Image Sequences.\n"
						"Expected: \"%ls\", Found: \"%ls\"", *Calib.CameraId, *Calib.CameraId, *ImageSequenceNamesString);
					}
				}
			}
		}

		FPerTakeResult TakeResult = FPerTakeResult(AssetsData.TakeId, Result);
		InPerTakeCallback(MoveTemp(TakeResult));
	}
}

FIngestAssetCreator::FAssetCreationResult FIngestAssetCreator::CheckCreatedTakeAssets_GameThread(const FCreateAssetsData& InCreateAssetsData)
{
	IFileManager& FileManager = IFileManager::Get();

	for (const FCreateAssetsData::FImageSequenceData& ImageSequenceData : InCreateAssetsData.ImageSequences)
	{
		UImgMediaSource* ImageSequence = GetAssetIfExists<UImgMediaSource>(InCreateAssetsData.PackagePath, ImageSequenceData.AssetName);
		CHECK_UOBJECT_RET(ImageSequence, ImageSequenceData.AssetName);

		FString ImageSequencePath = ImageSequence->GetFullPath();

		CHECK_RET(FPaths::DirectoryExists(ImageSequencePath), LOCTEXT("CheckCreatedTakeAssets_ImageDirNotFound", "Image Sequence directory doesn't exist"));

		TArray<FString> ImageSequenceFiles;
		CHECK_RET(FImageSequenceUtils::GetImageSequenceFilesFromPath(*ImageSequencePath, ImageSequenceFiles), LOCTEXT("CheckCreatedTakeAssets_MissingImageFiles", "No supported image files could be found"));
	}

	for (const FCreateAssetsData::FImageSequenceData& DepthSequenceData : InCreateAssetsData.DepthSequences)
	{
		UImgMediaSource* DepthSequence = GetAssetIfExists<UImgMediaSource>(InCreateAssetsData.PackagePath, DepthSequenceData.AssetName);
		CHECK_UOBJECT_RET(DepthSequence, DepthSequenceData.AssetName);

		FString DepthSequencePath = DepthSequence->GetFullPath();
		CHECK_RET(FPaths::DirectoryExists(DepthSequencePath), LOCTEXT("CheckCreatedTakeAssets_DepthDirNotFound", "Depth Sequence directory doesn't exist"));

		TArray<FString> DepthSequenceFiles;
		CHECK_RET(FImageSequenceUtils::GetImageSequenceFilesFromPath(*DepthSequencePath, DepthSequenceFiles), LOCTEXT("CheckCreatedTakeAssets_MissingDepthFiles", "No supported depth files could be found"));
	}

	for (const FCreateAssetsData::FAudioData& AudioData : InCreateAssetsData.AudioClips)
	{
		UObject* SoundWave = GetAssetIfExists(InCreateAssetsData.PackagePath, AudioData.AssetName);
		CHECK_UOBJECT_RET(SoundWave, AudioData.AssetName);
	}

	for (const FCreateAssetsData::FCalibrationData& Calibration : InCreateAssetsData.Calibrations)
	{
		UCameraCalibration* CalibrationAsset = GetAssetIfExists<UCameraCalibration>(InCreateAssetsData.PackagePath, Calibration.AssetName);
		CHECK_UOBJECT_RET(CalibrationAsset, Calibration.AssetName);

		for (const FExtendedLensFile& ExtendedLensFile : CalibrationAsset->CameraCalibrations)
		{
			CHECK_UOBJECT_RET(ExtendedLensFile.LensFile, TEXT("LensFile"));
		}
	}

	return MakeValue();
}

FIngestAssetCreator::FAssetCreationResult FIngestAssetCreator::CheckCreatedTakeStruct_GameThread(const FCaptureDataAssetInfo& InCreatedTakeStruct)
{
	for (const FCaptureDataAssetInfo::FImageSequence& ImageSequence : InCreatedTakeStruct.ImageSequences)
	{
		CHECK_RET(ImageSequence.Asset, LOCTEXT("AssetCreationError_ImageSequenceNotLinked", "Image Sequence asset not linked to the take"));
	}

	for (const FCaptureDataAssetInfo::FImageSequence& DepthSequence : InCreatedTakeStruct.DepthSequences)
	{
		CHECK_WARN(DepthSequence.Asset, LOCTEXT("AssetCreationError_DepthSequenceNotLinked", "Depth Sequence asset not linked to the take"));
	}
	
	for (const FCaptureDataAssetInfo::FAudio& Audio : InCreatedTakeStruct.Audios)
	{
		CHECK_RET(Audio.Asset, LOCTEXT("AssetCreationError_SoundWaveNotLinked", "Sound Wave asset not linked to the take"));
	}

	for (const FCaptureDataAssetInfo::FCalibration& Calibration : InCreatedTakeStruct.Calibrations)
	{
		CHECK_RET(Calibration.Asset, LOCTEXT("AssetCreationError_CameraCalibrationNotLinked", "Camera Calibration asset not linked to the take"));
	}

	return MakeValue();
}

template<typename ArrayElement>
int32 FindArrayIndexByTakeId(const TArray<ArrayElement>& InArray, int32 InTakeId)
{
	static_assert(TIsClass<ArrayElement>::Value, "Array element must be a struct/class");
	static_assert(TIsIntegral<decltype(ArrayElement::TakeId)>::Value, "Array element must contain TakeId member");

	return InArray.IndexOfByPredicate([InTakeId](const ArrayElement& InElem)
	{
		return InElem.TakeId == InTakeId;
	});
}

void FIngestAssetCreator::RemoveTakes(TArray<int32> InTakesToRemove,
											TArray<FCaptureDataAssetInfo>& OutTakeList)
{
	for (int32 TakeId : InTakesToRemove)
	{
		const int32 ItemIndex = FindArrayIndexByTakeId(OutTakeList, TakeId);

		if (ItemIndex != INDEX_NONE)
		{
			OutTakeList.RemoveAt(ItemIndex);
		}
	}
}

FIngestAssetCreator::FAssetCreationResult FIngestAssetCreator::CheckTakeAssets(const FCaptureDataAssetInfo& InTake)
{
	TArray<FText> Messages;

	for (const FCaptureDataAssetInfo::FAudio& Audio : InTake.Audios)
	{
		if (!Audio.Asset)
		{
			FText Message = LOCTEXT("AssetCreationError_ImportingAudioAssetsFailed", "Error importing audio assets");
			return MakeError(FAssetCreationError(MoveTemp(Message)));
		}
	}

	return MakeValue();
}

bool FIngestAssetCreator::RemoveAssetsByPath(const FString& InPackagePath)
{
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	TArray<FAssetData> AssetData;
	AssetRegistry.GetAssetsByPath(*(InPackagePath), AssetData);

	if (AssetData.IsEmpty())
	{
		return true;
	}

	const int32 DeletedCount = ObjectTools::DeleteAssets(AssetData, false);
	if (DeletedCount != AssetData.Num())
	{
		UE_LOGF(LogIngestAssetCreator, Warning, "RemoveAssetsByPath: deleted %d of %d assets at '%ls'",
			DeletedCount, AssetData.Num(), *InPackagePath
		);
		return false;
	}

	return true;
}

UObject* FIngestAssetCreator::GetAssetIfExists(const FString& InTargetPackagePath, const FString& InAssetName)
{
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	TArray<FAssetData> AssetData;
	AssetRegistry.GetAssetsByPackageName(*(InTargetPackagePath / InAssetName), AssetData);

	return !AssetData.IsEmpty() ? AssetData[0].GetAsset() : nullptr;
}

UObject* FIngestAssetCreator::GetOrCreateAsset(const FString& InTargetPackagePath, const FString& InAssetName, UClass* InClass)
{
	if (UObject* FoundAsset = GetAssetIfExists(InTargetPackagePath, InAssetName))
	{
		return FoundAsset;
	}
	else
	{
		return CreateAsset(InTargetPackagePath, InAssetName, InClass);
	}
}

UObject* FIngestAssetCreator::CreateAsset(const FString& InTargetPackagePath, const FString& InAssetName, UClass* InClass)
{
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>(TEXT("AssetTools")).Get();
	return AssetTools.CreateAsset(InAssetName, InTargetPackagePath, InClass, nullptr);
}

}

#undef LOCTEXT_NAMESPACE