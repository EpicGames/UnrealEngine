// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

#include "HAL/Platform.h"

#define UE_API DATAINGESTCOREEDITOR_API

/** Asset creation error types. */
enum class EAssetCreationError
{
	InternalError = 1,
	InvalidArgument,
	NotFound,
	Warning,
};

/** Information related to an error during ingest asset creation. */
class FAssetCreationError
{
public:

	UE_API FAssetCreationError(FText InMessage, EAssetCreationError InError = EAssetCreationError::InternalError);

	/** Get the error message. */
	UE_API const FText& GetMessage() const;

	/** Get the error type. */
	UE_API EAssetCreationError GetError() const;

private:

	/** Error message. */
	FText Message;

	/** Error type. */
	EAssetCreationError Error;
};

#undef UE_API
