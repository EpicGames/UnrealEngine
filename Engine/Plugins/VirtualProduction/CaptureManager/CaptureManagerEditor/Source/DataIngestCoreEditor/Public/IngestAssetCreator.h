// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrameRange.h"
#include "AssetImportTask.h"

#include "DataIngestCoreError.h"

#include "Async/ManagedDelegate.h"

#include "Misc/Timecode.h"

#include "CaptureData.h"
#include "CameraCalibration.h"

#define UE_API DATAINGESTCOREEDITOR_API

namespace UE::CaptureManager
{

/** Data used to create assets. */
struct FCreateAssetsData
{
	/** Image sequence information. */
	struct FImageSequenceData
	{
		FString AssetName;

		FString Name;
		FString SequenceDirectory;
		FFrameRate FrameRate = FFrameRate(30, 1);

		/** Whether timecode is available. */
		bool bTimecodePresent = false;

		FTimecode Timecode = FTimecode(0, 0, 0, 0, false);
		FFrameRate TimecodeRate = FFrameRate(30, 1);
	};

	/** Audio information. */
	struct FAudioData
	{
		FString AssetName;

		FString Name;
		FString WAVFile;

		/** Whether timecode is available. */
		bool bTimecodePresent = false;

		FTimecode Timecode = FTimecode(0, 0, 0, 0, false);
		FFrameRate TimecodeRate = FFrameRate(30, 1);
	};

	/** Calibration information. */
	struct FCalibrationData
	{
		FString AssetName;

		FString Name = TEXT("Calibration");
		TArray<FCameraCalibration> CameraCalibrations;
		TMap<FString, FString> LensFileAssetNames;
	};

	FString CaptureDataAssetName;

	/** Unique identifier for this take. */
	int32 TakeId = 0;
	FString PackagePath;
	TArray<FImageSequenceData> ImageSequences;
	TArray<FImageSequenceData> DepthSequences;
	TArray<FAudioData> AudioClips;
	TArray<FCalibrationData> Calibrations;

	/** Frame range(s) to exclude from processing. */
	TArray<FFrameRange> CaptureExcludedFrames;
};

/** Capture data asset information. */
struct FCaptureDataAssetInfo
{
	/** Image sequence information. */
	struct FImageSequence
	{
		TObjectPtr<UImgMediaSource> Asset;
		FTimecode Timecode;
		FFrameRate TimecodeRate;
	};

	/** Calibration information. */
	struct FCalibration
	{
		TObjectPtr<UCameraCalibration> Asset;
	};

	/** Audio information. */
	struct FAudio
	{
		TObjectPtr<USoundWave> Asset;
		FTimecode Timecode;
		FFrameRate TimecodeRate;
	};

	/** Unique identifier for this take. */
	int32 TakeId;

	TArray<FImageSequence> ImageSequences;
	TArray<FImageSequence> DepthSequences;
	TArray<FCalibration> Calibrations;
	TArray<FAudio> Audios;

	/** Frame range(s) to exclude from processing. */
	TArray<FFrameRange> CaptureExcludedFrames;
};

/** Facilitates creation, retrieval and removal of capture data assets. */
class FIngestAssetCreator
{
public:
	using FAssetCreationResult = TValueOrError<void, FAssetCreationError>;
	using FPerTakeResult = TPair<int32, FAssetCreationResult>;
	using FPerTakeCallback = TManagedDelegate<FPerTakeResult>;

	/** Creates assets from asset data. */
	static UE_API TArray<FCaptureDataAssetInfo> CreateAssets_GameThread(const TArray<FCreateAssetsData>& InOutCreateAssetDataList,
																 FPerTakeCallback InPerTakeCallback);

	/** Gets an asset if it exists. */
	static UE_API UObject* GetAssetIfExists(const FString& InTargetPackagePath, const FString& InAssetName);

	template<class TReturnType>
	static TReturnType* GetAssetIfExists(const FString& InTargetPackagePath, const FString& InAssetName)
	{
		return Cast<TReturnType>(GetAssetIfExists(InTargetPackagePath, InAssetName));
	}

	/** Gets an existing asset or create a new asset at the specified package path. */
	template<class TReturnType>
	static TReturnType* GetOrCreateAsset(const FString& InTargetPackagePath, const FString& InAssetName)
	{
		return Cast<TReturnType>(GetOrCreateAsset(InTargetPackagePath, InAssetName, TReturnType::StaticClass()));
	}

	/** Creates a new asset at the specified package path. */
	template<class TReturnType>
	static TReturnType* CreateAsset(const FString& InTargetPackagePath, const FString& InAssetName)
	{
		return Cast<TReturnType>(CreateAsset(InTargetPackagePath, InAssetName, TReturnType::StaticClass()));
	}

private:
	static UE_API void CreateTakeAssets_GameThread(const TArray<FCreateAssetsData>& InOutCreateAssetsData,
											const FPerTakeCallback& InPerTakeCallback,
											TArray<FCaptureDataAssetInfo>& OutTakes);

	static UE_API void VerifyIngestedData_GameThread(const TArray<FCreateAssetsData>& InCreateAssetsData,
											  const TArray<FCaptureDataAssetInfo>& InCreatedTakes,
											  const FPerTakeCallback& InPerTakeCallback);

	static UE_API TValueOrError<void, FText> CreateTakeAssetViews_GameThread(const FCreateAssetsData& InCreateAssetDate,
												FCaptureDataAssetInfo& OutTake);

	/** Adds audio asset to capture data asset info. */
	static UE_API FAssetCreationResult AssignAudioAsset(const FCreateAssetsData::FAudioData& AudioClip,
												 const TObjectPtr<UAssetImportTask>& InAssetImportTask,
												 FCaptureDataAssetInfo& OutTake);

	/** Deletes all assets at the specified package path.
	 *  Returns true if all assets were deleted (or none existed), false if any deletion failed. */
	static UE_API bool RemoveAssetsByPath(const FString& InPackagePath);

	/** Converts timecode to string. */
	static UE_API FString CreateTimecodeString(FTimecode InTimecode, FFrameRate InFrameRate);

	/** Creates asset path from import task. */
	static UE_API FString CreateAssetPathString(const UAssetImportTask* InAssetImportTask);

	/** Adds timecode to soundwave asset. */
	static UE_API void StampWithTakeMetadataTimecode(const FCreateAssetsData::FAudioData& InAudioClip, const UAssetImportTask* InAssetImportTask, USoundWave* OutSoundWave);
	
	/** Checks whether timecode rate is valid. */
	static UE_API bool IsValidAudioTimecodeRate(FFrameRate InTimecodeRate, uint32 InNumSamplesPerSecond);
	
	/** Creates sound wave asset. */
	static UE_API void PrepareSoundWave(const FCreateAssetsData::FAudioData& InAudioClip, const UAssetImportTask* InAssetImportTask, USoundWave* OutSoundWave);
	

	static UE_API FAssetCreationResult CheckCreatedTakeAssets_GameThread(const FCreateAssetsData& InCreateAssetsData);
	static UE_API FAssetCreationResult CheckCreatedTakeStruct_GameThread(const FCaptureDataAssetInfo& InCreatedTakeStruct);

	/** Deletes specified takes from take list. */
	static UE_API void RemoveTakes(TArray<int32> InTakesToRemove,
								  TArray<FCaptureDataAssetInfo>& OutTakeList);

	/** Checks validity of assets in the take. */
	static UE_API FAssetCreationResult CheckTakeAssets(const FCaptureDataAssetInfo& InTake);

	static UE_API UObject* GetOrCreateAsset(const FString& InTargetPackagePath, const FString& InAssetName, UClass* InClass);
	static UE_API UObject* CreateAsset(const FString& InTargetPackagePath, const FString& InAssetName, UClass* InClass);

	static UE_API TValueOrError<TObjectPtr<class UImgMediaSource>, FText> CreateImageSequenceAsset(const FString& InPackagePath,
																	  const FCreateAssetsData::FImageSequenceData& InImageSequenceData);

};
}

#undef UE_API
