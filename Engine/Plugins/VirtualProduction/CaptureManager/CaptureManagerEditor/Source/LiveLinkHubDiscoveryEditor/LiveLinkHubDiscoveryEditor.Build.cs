// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkHubDiscoveryEditor : ModuleRules
{
	public LiveLinkHubDiscoveryEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] 
		{
			"CaptureUtils",
			"Core",
			"CoreUObject",
			"LiveLinkHubCaptureMessaging",
			"LiveLinkHubWorkerManager",
			"LiveLinkHubExportServer",
			"Messaging",
			"Sockets",
		});

		ShortName = "LLHubDiscEd";
	}
}
