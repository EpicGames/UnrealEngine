// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiscoveryResponder.h"

#include "LiveLinkHubExportServerModule.h"
#include "LiveLinkHubWorkerManagerModule.h"

#include "MessageEndpointBuilder.h"

#include "Misc/CoreDelegates.h"

#include "Network/NetworkMisc.h"

DEFINE_LOG_CATEGORY(LogLiveLinkHubDiscoveryEditor)

namespace UE::CaptureManager::Private
{
static FString GetExportServerErrorMessage(const FLiveLinkHubExportServer::EServerError InServerError)
{
	using EServerError = FLiveLinkHubExportServer::EServerError;

	switch (InServerError)
	{
	case EServerError::NotRunning:
		return TEXT("Server not running");

	case EServerError::InvalidIPAddress:
		return TEXT("Invalid IP address");

	case EServerError::InvalidPort:
		return TEXT("Invalid port");

	default:
		check(false);
		return TEXT("Unknown error");
	}
}
} // UE::CaptureManager::Private

namespace UE::CaptureManager
{

TValueOrError<FLiveLinkHubExportServer::FServerInfo, FLiveLinkHubExportServer::EServerError> FDiscoveryResponder::GetExportServerInfo()
{
	FLiveLinkHubExportServerModule& LiveLinkHubExportServerModule = FModuleManager::LoadModuleChecked<FLiveLinkHubExportServerModule>("LiveLinkHubExportServer");
	return LiveLinkHubExportServerModule.GetExportServerInfo();
}

TSharedRef<FLiveLinkHubWorkerManager> FDiscoveryResponder::GetWorkerManager()
{
	FLiveLinkHubWorkerManagerModule& LiveLinkHubWorkerManagerModule = FModuleManager::LoadModuleChecked<FLiveLinkHubWorkerManagerModule>("LiveLinkHubWorkerManager");
	return LiveLinkHubWorkerManagerModule.GetManager();
}


FDiscoveryResponder::FDiscoveryResponder()
	: Manager(GetWorkerManager())
	, bStarted(false)
{
	using FServerInfo = FLiveLinkHubExportServer::FServerInfo;
	using EServerError = FLiveLinkHubExportServer::EServerError;

	HostName = GetLocalHostNameChecked();
	MessageEndpoint = FMessageEndpoint::Builder("DiscoveryResponder")
		.Handling<FDiscoveryRequest>(this, &FDiscoveryResponder::HandleDiscoveryRequest)
		.ReceivingOnAnyThread()
		;

	MessageEndpoint->Subscribe<FDiscoveryRequest>();

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FDiscoveryResponder::StartDiscoveryResponder);
}

FDiscoveryResponder::~FDiscoveryResponder()
{
	MessageEndpoint->Unsubscribe();

	FMessageEndpoint::SafeRelease(MessageEndpoint);
}

void FDiscoveryResponder::HandleDiscoveryRequest(
	const FDiscoveryRequest& InRequest,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	if (!bStarted)
	{
		return;
	}

	if (InRequest.MessageVersion != 1)
	{
		return;
	}

	using FServerInfo = FLiveLinkHubExportServer::FServerInfo;
	using EServerError = FLiveLinkHubExportServer::EServerError;

	if (!CachedServerInfo.IsSet())
	{
		TValueOrError<FServerInfo, EServerError> Info = GetExportServerInfo();
		if (Info.HasError())
		{
			CachedServerInfo.Reset();
			return;
		}

		CachedServerInfo = Info.StealValue();
	}

	FDiscoveryResponse* Response = FMessageEndpoint::MakeMessage<FDiscoveryResponse>();
	Response->ExportPort = CachedServerInfo.GetValue().Port;
	Response->IPAddress = CachedServerInfo.GetValue().IPAddress;
	Response->HostName = HostName;

	Manager->SendDiscoveryResponse(Response, InContext->GetSender());

	UE_LOGF(LogLiveLinkHubDiscoveryEditor, Verbose, "Discovery request from %ls with endpoint ID: %ls", *InRequest.HostName, *InContext->GetSender().ToString());
}

void FDiscoveryResponder::StartDiscoveryResponder()
{
	bStarted = true;

	TValueOrError<FLiveLinkHubExportServer::FServerInfo, FLiveLinkHubExportServer::EServerError> Info = GetExportServerInfo();
	if (Info.HasError())
	{
		CachedServerInfo.Reset();
		return;
	}

	CachedServerInfo = Info.StealValue();
}

} // namespace UE::CaptureManager
