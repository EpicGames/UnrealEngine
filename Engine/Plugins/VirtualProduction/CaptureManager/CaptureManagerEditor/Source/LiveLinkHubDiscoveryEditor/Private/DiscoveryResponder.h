// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

#include "LiveLinkHubCaptureMessages.h"
#include "LiveLinkHubExportServer.h"
#include "LiveLinkHubWorkerManager.h"

#include "MessageEndpoint.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkHubDiscoveryEditor, Log, All)

namespace UE::CaptureManager
{

class FDiscoveryResponder
{
public:
	FDiscoveryResponder();
	~FDiscoveryResponder();

private:

	static TValueOrError<FLiveLinkHubExportServer::FServerInfo, FLiveLinkHubExportServer::EServerError> GetExportServerInfo();
	static TSharedRef<FLiveLinkHubWorkerManager> GetWorkerManager();

	void HandleDiscoveryRequest(
		const FDiscoveryRequest& InRequest,
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
	);

	void StartDiscoveryResponder();

	TSharedRef<FLiveLinkHubWorkerManager> Manager;

	FString HostName;
	TOptional<FLiveLinkHubExportServer::FServerInfo> CachedServerInfo;

	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	std::atomic_bool bStarted = false;
};

} // namespace UE::CaptureManager