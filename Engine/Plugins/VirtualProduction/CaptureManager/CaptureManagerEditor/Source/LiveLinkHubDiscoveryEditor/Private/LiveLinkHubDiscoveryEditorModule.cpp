// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubDiscoveryEditorModule.h"

#include "DiscoveryResponder.h"

void FLiveLinkHubDiscoveryEditor::StartupModule()
{
	DiscoveryResponder = MakeUnique<UE::CaptureManager::FDiscoveryResponder>();
}
void FLiveLinkHubDiscoveryEditor::ShutdownModule()
{
	DiscoveryResponder.Reset();
}

IMPLEMENT_MODULE(FLiveLinkHubDiscoveryEditor, LiveLinkHubDiscoveryEditor);