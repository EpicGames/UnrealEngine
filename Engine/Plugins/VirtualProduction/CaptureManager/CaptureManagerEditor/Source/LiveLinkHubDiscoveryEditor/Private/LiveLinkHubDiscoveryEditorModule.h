// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

namespace UE::CaptureManager
{
class FDiscoveryResponder;
}

class FLiveLinkHubDiscoveryEditor : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TUniquePtr<UE::CaptureManager::FDiscoveryResponder> DiscoveryResponder;
};
