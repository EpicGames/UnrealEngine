// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkHubExportServer : ModuleRules
{
	public LiveLinkHubExportServer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"CaptureUtils"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Sockets",
			"LiveLinkHubCaptureMessaging"
		});


		ShortName = "LLHubExpS";
	}
}
