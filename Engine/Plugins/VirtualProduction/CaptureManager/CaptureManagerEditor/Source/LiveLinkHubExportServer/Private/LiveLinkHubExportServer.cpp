// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubExportServer.h"

#include "HAL/Runnable.h"
#include "Network/NetworkMisc.h"
#include "Network/TcpServer.h"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkHubExportServer, Log, All);

namespace UE::CaptureManager::Private
{
static constexpr int64 MaxNumberOfClients = 20;
static constexpr uint16 DefaultExportServerPort = 0;
}

class FLiveLinkHubExportServer::FLiveLinkHubClientExportRunner final : public FRunnable
{
public:

	FLiveLinkHubClientExportRunner(FLiveLinkHubExportServer& InExportServer,
		TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient);

private:

	virtual uint32 Run() override;
	virtual void Stop() override;

	FLiveLinkHubExportServer& ExportServer;
	TSharedPtr<UE::CaptureManager::FTcpClientHandler> Client;
	std::atomic_bool bIsRunning;
};

FLiveLinkHubExportServer::FLiveLinkHubExportServer()
	: TcpServer(MakeShared<UE::CaptureManager::FTcpServer>(UE::CaptureManager::Private::MaxNumberOfClients))
{
	TcpServer->SetConnectionHandler(UE::CaptureManager::FTcpServer::FConnectionHandler::CreateRaw(this, &FLiveLinkHubExportServer::OnConnectionChanged));
}

FLiveLinkHubExportServer::~FLiveLinkHubExportServer() = default;

bool FLiveLinkHubExportServer::Start()
{
	return Start(UE::CaptureManager::Private::DefaultExportServerPort);
}

bool FLiveLinkHubExportServer::Start(uint16 InPort)
{
	using namespace UE::CaptureManager;

	TProtocolResult<uint16> SuccessPort = TcpServer->Start(InPort);

	if (SuccessPort.HasValue())
	{
		UE_LOGF(LogLiveLinkHubExportServer, Display, "LiveLink Hub ingest server running on port %d", SuccessPort.GetValue());
	}

	return SuccessPort.HasValue();
}

bool FLiveLinkHubExportServer::Stop()
{
	using namespace UE::CaptureManager;

	TProtocolResult<void> StopResult = TcpServer->Stop();

	if (StopResult.HasValue())
	{
		UE_LOGF(LogLiveLinkHubExportServer, Display, "LiveLink Hub ingest server stopped");
	}

	return StopResult.HasValue();
}

bool FLiveLinkHubExportServer::IsRunning() const
{
	return (TcpServer && TcpServer->IsRunning());
}

TValueOrError<FLiveLinkHubExportServer::FServerInfo, FLiveLinkHubExportServer::EServerError> FLiveLinkHubExportServer::GetServerInfo() const
{
	if (!(TcpServer && TcpServer->IsRunning()))
	{
		return MakeError(EServerError::NotRunning);
	}

	const int32 PortNumberSigned = TcpServer->GetPort();
	const bool bPortIsInRange = PortNumberSigned >= TNumericLimits<uint16>::Min() && PortNumberSigned <= TNumericLimits<uint16>::Max();

	if (!ensure(bPortIsInRange))
	{
		return MakeError(EServerError::InvalidPort);
	}

	const uint16 PortNumber = static_cast<uint16>(PortNumberSigned);

	TOptional<FString> LocalIPAddress = UE::CaptureManager::GetLocalIpAddress();

	if (!LocalIPAddress)
	{
		return MakeError(EServerError::InvalidIPAddress);
	}

	FServerInfo ServerInfo
	{
		.IPAddress = MoveTemp(*LocalIPAddress),
		.Port = PortNumber
	};

	return MakeValue(MoveTemp(ServerInfo));
}

void FLiveLinkHubExportServer::RegisterFileDownloadHandler(FString InClientId, FFileDataHandler InFileDataHandler)
{
	FScopeLock Lock(&HandlersMutex);

	Handlers.Add(MoveTemp(InClientId), MoveTemp(InFileDataHandler));
}

void FLiveLinkHubExportServer::UnregisterFileDownloadHandler(FString InClientId)
{
	FScopeLock Lock(&HandlersMutex);

	if (Handlers.Contains(InClientId))
	{
		Handlers.Remove(InClientId);
	}
}

void FLiveLinkHubExportServer::OnConnectionChanged(TWeakPtr<UE::CaptureManager::FTcpClientHandler> InClient, bool bIsConnected)
{
	using namespace UE::CaptureManager;

	FScopeLock Lock(&Mutex);

	TSharedPtr<FTcpClientHandler> Client = InClient.Pin();

	const FString& Endpoint = Client->GetEndpoint();
	if (bIsConnected)
	{
		TUniquePtr<FLiveLinkHubClientExportRunner>& Runner = Runners.Add(Endpoint, MakeUnique<FLiveLinkHubClientExportRunner>(*this, Client));
		
		TUniquePtr<FRunnableThread> Thread;
		Thread.Reset(FRunnableThread::Create(Runner.Get(), TEXT("Upload Data Runner"), 128 * 1024, TPri_Normal, FPlatformAffinity::GetPoolThreadMask()));
		
		Threads.Add(Endpoint, MoveTemp(Thread));
	}
	else
	{
		checkf(Threads.Contains(Endpoint), TEXT("Client can't be disconnected as it doesn't exist."));

		TUniquePtr<FRunnableThread>& Thread = Threads[Endpoint];
		Thread->Kill(true);

		Threads.Remove(Endpoint);
		Runners.Remove(Endpoint);
	}
}

bool FLiveLinkHubExportServer::HandleFileData(TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient, FUploadDataHeader Header)
{
	FString ClientId = Header.ClientId.ToString();

	FFileDataHandler Handler;

	{
		FScopeLock Lock(&HandlersMutex);
		FFileDataHandler* FoundHandler = Handlers.Find(ClientId);

		if (!FoundHandler)
		{
			return false;
		}

		Handler = *FoundHandler;
	}

	return Handler.Execute(MoveTemp(Header), MoveTemp(InClient));
}

FLiveLinkHubExportServer::FLiveLinkHubClientExportRunner::FLiveLinkHubClientExportRunner(FLiveLinkHubExportServer& InExportServer,
																						 TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient)
	: ExportServer(InExportServer)
	, Client(InClient)
	, bIsRunning(true)
{
}

uint32 FLiveLinkHubExportServer::FLiveLinkHubClientExportRunner::Run()
{
	using namespace UE::CaptureManager;

	check(Client);
	check(ExportServer.TcpServer);

	bool bLocalRunning = true;

	bIsRunning.store(bLocalRunning);

	while (bLocalRunning)
	{
		FTcpConnectionReader Reader(*Client);
		FUploadResult<FUploadDataHeader> HeaderResult = FUploadDataMessage::DeserializeHeader(Reader);
		if (HeaderResult.HasError())
		{
			Async(EAsyncExecution::LargeThreadPool,
				[Server = ExportServer.TcpServer, Endpoint = Client->GetEndpoint()]()
				{
					Server->DisconnectClient(Endpoint);
				}
			);
			break;
		}

		FUploadDataHeader Header = HeaderResult.StealValue();
		
		// Ready for the data to be sent to the correct handler
		const bool bFileDataHandled = ExportServer.HandleFileData(Client, MoveTemp(Header));

		if (!bFileDataHandled)
		{
			Async(EAsyncExecution::LargeThreadPool,
				[Server = ExportServer.TcpServer, Endpoint = Client->GetEndpoint()]()
				{
					Server->DisconnectClient(Endpoint);
				}
			);
			break;
		}

		bLocalRunning = bIsRunning.load();
	}
	
	Stop();
	return 0;
}

void FLiveLinkHubExportServer::FLiveLinkHubClientExportRunner::Stop()
{
	bIsRunning.store(false);
}