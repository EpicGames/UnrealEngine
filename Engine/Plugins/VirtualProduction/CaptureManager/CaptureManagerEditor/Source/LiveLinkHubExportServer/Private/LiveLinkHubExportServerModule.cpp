// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubExportServerModule.h"

void FLiveLinkHubExportServerModule::StartupModule()
{
	ExportServer = MakeShared<FLiveLinkHubExportServer>();
}

void FLiveLinkHubExportServerModule::ShutdownModule()
{
	StopExportServer();
	ExportServer = nullptr;
}

bool FLiveLinkHubExportServerModule::StartExportServer(uint16 InPort)
{
	return ExportServer->Start(InPort);
}

bool FLiveLinkHubExportServerModule::StopExportServer()
{
	return ExportServer->Stop();
}

bool FLiveLinkHubExportServerModule::IsExportServerRunning() const
{
	return ExportServer->IsRunning();
}

TValueOrError<FLiveLinkHubExportServer::FServerInfo, FLiveLinkHubExportServer::EServerError> FLiveLinkHubExportServerModule::GetExportServerInfo() const
{
	return ExportServer->GetServerInfo();
}

void FLiveLinkHubExportServerModule::RegisterExportServerHandler(FString InClientId, 
																 FLiveLinkHubExportServer::FFileDataHandler InFileDataHandler)
{
	ExportServer->RegisterFileDownloadHandler(MoveTemp(InClientId), MoveTemp(InFileDataHandler));
}

void FLiveLinkHubExportServerModule::UnregisterExportServerHandler(FString InClientId)
{
	ExportServer->UnregisterFileDownloadHandler(MoveTemp(InClientId));
}


IMPLEMENT_MODULE(FLiveLinkHubExportServerModule, LiveLinkHubExportServer)