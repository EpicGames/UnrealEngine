// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UploadDataMessage.h"

#define UE_API LIVELINKHUBEXPORTSERVER_API

namespace UE::CaptureManager
{
class FTcpServer;
class FTcpClientHandler;
}

class FLiveLinkHubExportServer
{
public:

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FFileDataHandler, FUploadDataHeader, TSharedPtr<UE::CaptureManager::FTcpClientHandler>)

	struct FServerInfo
	{
		FString IPAddress;
		uint16 Port = 0;
	};

	enum class EServerError
	{
		NotRunning,
		InvalidPort,
		InvalidIPAddress
	};

	UE_API FLiveLinkHubExportServer();
	UE_API ~FLiveLinkHubExportServer();

	UE_API bool Start();
	UE_API bool Start(uint16 InPort);
	UE_API bool Stop();

	UE_API bool IsRunning() const;

	UE_API TValueOrError<FServerInfo, EServerError> GetServerInfo() const;

	UE_API void RegisterFileDownloadHandler(FString InClientId, FFileDataHandler InFileDataHandler);
	UE_API void UnregisterFileDownloadHandler(FString InClientId);

private:

	class FLiveLinkHubClientExportRunner;

	UE_API void OnConnectionChanged(TWeakPtr<UE::CaptureManager::FTcpClientHandler> InClient, bool bIsConnected);
	UE_API bool HandleFileData(TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient, FUploadDataHeader Header);

	TSharedPtr<UE::CaptureManager::FTcpServer> TcpServer;

	FCriticalSection Mutex;
	FCriticalSection HandlersMutex;
	TMap<FString, FFileDataHandler> Handlers; // The key is stringified FGuid
	TMap<FString, TUniquePtr<FLiveLinkHubClientExportRunner>> Runners;
	TMap<FString, TUniquePtr<FRunnableThread>> Threads;
};

#undef UE_API
