// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#include "LiveLinkHubExportServer.h"

#define UE_API LIVELINKHUBEXPORTSERVER_API

class FLiveLinkHubExportServerModule
	: public IModuleInterface
{
public:

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	UE_API bool StartExportServer(uint16 InPort);
	UE_API bool StopExportServer();
	UE_API bool IsExportServerRunning() const;

	UE_API TValueOrError<FLiveLinkHubExportServer::FServerInfo, FLiveLinkHubExportServer::EServerError> GetExportServerInfo() const;

	UE_API void RegisterExportServerHandler(FString InClientId, FLiveLinkHubExportServer::FFileDataHandler InFileDataHandler);
	UE_API void UnregisterExportServerHandler(FString InClientId);

private:

	TSharedPtr<FLiveLinkHubExportServer> ExportServer;
};

#undef UE_API
