// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkHubWorkerManager : ModuleRules
{
	public LiveLinkHubWorkerManager(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"CaptureUtils"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"LiveLinkHubCaptureMessaging",
			"ContentBrowserData",
			"DeveloperSettings",
			"MessagingCommon",
			"LiveLinkHubExportServer",
			"DataIngestCore",
			"DataIngestCoreEditor",
			"CaptureDataCore",
			"CaptureDataUtils",
			"CaptureManagerTakeMetadata",
			"CaptureManagerEditorSettings",
			"Settings",
			"UnrealEd",
			"NamingTokens"
		});

		ShortName = "LLHubWorMan";
	}
}
