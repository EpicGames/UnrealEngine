// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestCaptureDataProcess.h"

#include "CaptureManagerIngestPreparation.h"
#include "HAL/PlatformFileManager.h"

#include "Utils/ParseTakeUtils.h"
#include "Utils/UnrealCalibrationParser.h"

#include "Asset/CaptureAssetSanitization.h"
#include "Async/HelperFunctions.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "GlobalNamingTokens.h"
#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"

#include "Settings/CaptureManagerEditorSettings.h"
#include "Settings/CaptureManagerEditorTemplateTokens.h"

DEFINE_LOG_CATEGORY_STATIC(LogIngestCaptureDataProcess, Log, All);

#define LOCTEXT_NAMESPACE "IngestCaptureDataProcess"

TOptional<FString> GetIngestDataFileName(FString InTakeStoragePath)
{
	IFileManager& FileManager = IFileManager::Get();

	FString FileName = TEXT("");
	FileManager.IterateDirectory(*InTakeStoragePath, [&FileName](const TCHAR* InFileOrDirectoryName, bool bInIsDirectory)
		{
			if (bInIsDirectory)
			{
				return true;
			}

			if (FPaths::GetExtension(InFileOrDirectoryName) == FIngestCaptureData::Extension)
			{
				FileName = InFileOrDirectoryName;
				return false;
			}

			return true;
		});

	if (FileName.IsEmpty())
	{
		return {};
	}

	return FileName;
}

TValueOrError<FIngestProcessResult, FText> FIngestCaptureDataProcess::StartIngestProcess(const FString& InTakeStoragePath,
																						 const FString& InDeviceName,
																						 const FGuid& InTakeUploadId)
{
	using namespace UE::CaptureManager;

	TOptional<FString> IngestDataFileName = GetIngestDataFileName(InTakeStoragePath);
	if (!IngestDataFileName.IsSet())
	{
		FText Message = FText::Format(LOCTEXT("StartIngestProcess_TakeFileMissing", "Ingest capture data file is not found: {0}"),
			FText::FromString(*InTakeStoragePath));
		UE_LOGF(LogIngestCaptureDataProcess, Error, "%ls", *Message.ToString());
		return MakeError(MoveTemp(Message));
	}

	const FString IngestCaptureDataFilePath = IngestDataFileName.GetValue();

	IngestCaptureData::FParseResult IngestCaptureDataParseResult = IngestCaptureData::ParseFile(IngestCaptureDataFilePath);
	if (IngestCaptureDataParseResult.HasError())
	{
		FText Message = FText::Format(LOCTEXT("StartIngestProcess_TakeMetadataFailure", "Failed to read capture data file metadata: {0} - {1}"),
			IngestCaptureDataParseResult.GetError(),
			FText::FromString(*InTakeStoragePath));

		UE_LOGF(LogIngestCaptureDataProcess, Error, "%ls", *Message.ToString());
		return MakeError(MoveTemp(Message));
	}

	FIngestCaptureData IngestCaptureData = IngestCaptureDataParseResult.GetValue();

	IngestCaptureData.MakePathsAbsolute(InTakeStoragePath);

	FCreateAssetsData AssetCreationData = PrepareAssetsData(InTakeUploadId, InDeviceName, IngestCaptureData);

	FIngestProcessResult IngestProcessResult;
	IngestProcessResult.TakeIngestPackagePath = AssetCreationData.PackagePath;
	IngestProcessResult.AssetsData.Add(AssetCreationData);

	IngestProcessResult.CaptureDataTakeInfo = BuildTakeInfo(AssetCreationData, IngestCaptureData);

	return MakeValue(MoveTemp(IngestProcessResult));

}

UE::CaptureManager::FCreateAssetsData FIngestCaptureDataProcess::PrepareAssetsData(
	const FGuid& InTakeUploadId,
	const FString& InDeviceName,
	const FIngestCaptureData& InIngestCaptureData
)
{
	using namespace UE::CaptureManager;

	FCreateAssetsData CreateAssetsData;

	CallOnGameThread(
		[&InIngestCaptureData, &InTakeUploadId, &InDeviceName, &CreateAssetsData]()
		{
			// Needs to be on the game thread as we're using naming tokens
			FAssetNamingStrategy AssetNamingStrategy(InIngestCaptureData, InTakeUploadId.ToString(), InDeviceName);
			CreateAssetsData = BuildAssetData(InIngestCaptureData, AssetNamingStrategy);
		}
	);

	return CreateAssetsData;
}

#undef LOCTEXT_NAMESPACE