// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

#include "Containers/UnrealString.h"

#include "IngestProcessData.h"
#include "IngestCaptureData.h"
#include "Async/StopToken.h"

class FIngestCaptureDataProcess
{
public:
	static TValueOrError<FIngestProcessResult, FText> StartIngestProcess(const FString& InTakeStoragePath,
																		 const FString& InDeviceName,
																		 const FGuid& InTakeUploadId);

private:

	static UE::CaptureManager::FCreateAssetsData PrepareAssetsData(const FGuid& InTakeUploadId,
																   const FString& InDeviceName,
																   const FIngestCaptureData& InIngestCaptureData);
};