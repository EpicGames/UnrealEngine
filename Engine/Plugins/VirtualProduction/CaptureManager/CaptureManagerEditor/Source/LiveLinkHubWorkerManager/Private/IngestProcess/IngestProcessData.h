// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerIngestPreparation.h"

struct FIngestProcessResult
{
	FString TakeIngestPackagePath;
	UE::CaptureManager::FCaptureDataTakeInfo CaptureDataTakeInfo;
	TArray<UE::CaptureManager::FCreateAssetsData> AssetsData;
};