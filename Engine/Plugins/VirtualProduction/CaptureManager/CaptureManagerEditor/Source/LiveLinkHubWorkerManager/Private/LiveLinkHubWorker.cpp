// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubWorker.h"

#include "LiveLinkHubWorkerLog.h"

#include "LiveLinkHubExportServerModule.h"

#include "MessageEndpointBuilder.h"

#include "Misc/SecureHash.h"
#include "Misc/FileHelper.h"

#include "HAL/PlatformFileManager.h"

#include "IngestProcess/IngestCaptureDataProcess.h"

#include "Async/HelperFunctions.h"
#include "Network/TcpServer.h"

#include "CaptureManagerIngestPreparation.h"
#include "IngestProcess/IngestProcessData.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "GlobalNamingTokens.h"
#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"

#include "Settings/CaptureManagerEditorSettings.h"
#include "Settings/CaptureManagerEditorTemplateTokens.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubWorker"

DEFINE_LOG_CATEGORY(LogLiveLinkHubWorker);

namespace UE::CaptureManager::Private
{
static void PublishProgressEvent(TSharedPtr<FLiveLinkHubImportWorker::FEditorMessenger> InMessenger,
	const FGuid& InCaptureSourceId,
	const FGuid& InTakeUploadId,
	double InProgress)
{
	InMessenger->SendUploadStateMessage(InCaptureSourceId, InTakeUploadId, InProgress);
}

static void PublishDoneEvent(TSharedPtr<FLiveLinkHubImportWorker::FEditorMessenger> InMessenger,
	const FGuid& InCaptureSourceId,
	const FGuid& InTakeUploadId,
	FString InMessage = TEXT(""),
	int32 InCode = 0)
{
	InMessenger->SendUploadDoneMessage(InCaptureSourceId, InTakeUploadId, MoveTemp(InMessage), InCode);
}

TSharedPtr<FLiveLinkHubImportWorker> FLiveLinkHubImportWorker::Create(TWeakPtr<FEditorMessenger> InEditorMessenger)
{
	TSharedPtr<FLiveLinkHubImportWorker> SharedWorker = 
		MakeShared<FLiveLinkHubImportWorker>(MoveTemp(InEditorMessenger), FPrivateToken());

	TSharedPtr<FEditorMessenger> SharedMessenger = SharedWorker->Messenger.Pin();

	FLiveLinkHubExportServerModule& Module = FModuleManager::LoadModuleChecked<FLiveLinkHubExportServerModule>("LiveLinkHubExportServer");
	Module.RegisterExportServerHandler(SharedMessenger->GetAddress().ToString(),
									   FLiveLinkHubExportServer::FFileDataHandler::CreateSP(SharedWorker.ToSharedRef(), &FLiveLinkHubImportWorker::HandleTakeDownload));

	return SharedWorker;
}

FLiveLinkHubImportWorker::FLiveLinkHubImportWorker(TWeakPtr<FEditorMessenger> InEditorMessenger, FPrivateToken)
	: Messenger(MoveTemp(InEditorMessenger))
{
}

FLiveLinkHubImportWorker::~FLiveLinkHubImportWorker()
{
	FLiveLinkHubExportServerModule* Module = FModuleManager::GetModulePtr<FLiveLinkHubExportServerModule>(TEXT("LiveLinkHubExportServer"));

	if (Module)
	{
		TSharedPtr<FEditorMessenger> SharedMessenger = Messenger.Pin();

		if (SharedMessenger.IsValid())
		{
			Module->UnregisterExportServerHandler(SharedMessenger->GetAddress().ToString());
		}
	}
}

FString FLiveLinkHubImportWorker::EvaluateSettings(const FUploadDataHeader& InHeader)
{
	const UCaptureManagerEditorSettings* Settings = GetDefault<UCaptureManagerEditorSettings>();

	FStringFormatNamedArguments GeneralNamedArgs;
	{
		using namespace UE::CaptureManager;
		GeneralNamedArgs.Add(Settings->GetGeneralNamingTokens()->GetToken(FString(GeneralTokens::IdKey)).Name, InHeader.TakeUploadId.ToString());
		GeneralNamedArgs.Add(Settings->GetGeneralNamingTokens()->GetToken(FString(GeneralTokens::DeviceKey)).Name, InHeader.CaptureSourceName);
		GeneralNamedArgs.Add(Settings->GetGeneralNamingTokens()->GetToken(FString(GeneralTokens::SlateKey)).Name, InHeader.Slate);
		GeneralNamedArgs.Add(Settings->GetGeneralNamingTokens()->GetToken(FString(GeneralTokens::TakeKey)).Name, InHeader.TakeNumber);
	}						 

	FString DataStorageOut;
	// Naming tokens subsystem consults asset registry so need to run on the game thread
	CallOnGameThread(
		[&DataStorageOut, &GeneralNamedArgs, Settings]()
		{
			UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
			check(NamingTokensSubsystem);

			FNamingTokenFilterArgs NamingTokenArgs;
			if (const TObjectPtr<const UCaptureManagerIngestNamingTokens> Tokens = Settings->GetGeneralNamingTokens())
			{
				NamingTokenArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
			}

			// Evaluate Storage Paths
			FString MediaDirectory = FString::Format(*Settings->MediaDirectory.Path, GeneralNamedArgs);
			FNamingTokenResultData MediaDirectoryResult = NamingTokensSubsystem->EvaluateTokenString(MediaDirectory, NamingTokenArgs);
			DataStorageOut = MediaDirectoryResult.EvaluatedText.ToString();
		}
	);

	return DataStorageOut;
}

bool FLiveLinkHubImportWorker::HandleTakeDownload(FUploadDataHeader InHeader, TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient)
{
	using namespace UE::CaptureManager;

	TSharedPtr<FEditorMessenger> SharedMessenger = Messenger.Pin();

	if (!SharedMessenger.IsValid())
	{
		return false;
	}

	FString DataStorage = EvaluateSettings(InHeader);

	// Create the storage directory
	IFileManager& FileManager = IFileManager::Get();
	FileManager.MakeDirectory(*DataStorage, true);

	FGuid ClientId;
	FGuid::Parse(SharedMessenger->GetAddress().ToString(), ClientId);

	check(InHeader.ClientId == ClientId);

	FTakeFileContext Context;
	Context.TotalLength = InHeader.TotalLength;
	Context.RemainingLength = InHeader.TotalLength;

	FTcpConnectionReader Reader(*InClient);

	static constexpr int32 NumberOfTasks = 1;

	TSharedPtr<FTaskProgress> TaskProgress =
		MakeShared<FTaskProgress>(NumberOfTasks, FTaskProgress::FProgressReporter::CreateLambda([this, SharedMessenger, CaptureSourceId = InHeader.CaptureSourceId, TakeUploadId = InHeader.TakeUploadId](float InTotalProgress)
			{
				PublishProgressEvent(SharedMessenger, CaptureSourceId, TakeUploadId, InTotalProgress);
			}));
	TaskProgress->SetReportThreshold(3.0);

	FTaskProgress::FTask Task = TaskProgress->StartTask();

	while (Context.RemainingLength != 0)
	{
		FUploadResult<FUploadFileDataHeader> FileHeaderResult = FUploadDataMessage::DeserializeFileHeader(Reader);

		if (FileHeaderResult.HasError())
		{
			UE_LOGF(LogLiveLinkHubWorker, Error, "Failed to fetch file header: %ls", *FileHeaderResult.GetError().GetText().ToString());

			DeleteDownloadedData(DataStorage);

			FUploadError Error = FileHeaderResult.StealError();
			PublishDoneEvent(SharedMessenger, InHeader.CaptureSourceId, InHeader.TakeUploadId, Error.GetText().ToString(), Error.GetCode());

			// Closes the connection
			return false;
		}

		FUploadFileDataHeader FileHeader = FileHeaderResult.StealValue();

		FUploadVoidResult Result = HandleFileDownload(DataStorage, FileHeader, Context, InClient, Task);

		if (Result.HasError())
		{
			UE_LOGF(LogLiveLinkHubWorker, Error, "Failed to fetch file: %ls", *Result.GetError().GetText().ToString());

			DeleteDownloadedData(DataStorage);

			FUploadError Error = Result.StealError();
			PublishDoneEvent(SharedMessenger, InHeader.CaptureSourceId, InHeader.TakeUploadId, Error.GetText().ToString(), Error.GetCode());

			// Closes the connection
			return false;
		}

		Context.RemainingLength -= FileHeader.Length;
	}

	SpawnIngestTask(SharedMessenger, 
					DataStorage, 
					InHeader.CaptureSourceId, 
					InHeader.CaptureSourceName, 
					InHeader.TakeUploadId, 
					MoveTemp(TaskProgress));

	// Closes the connection
	return false;
}

FUploadVoidResult FLiveLinkHubImportWorker::HandleFileDownload(const FString& InTakeStoragaPath, const FUploadFileDataHeader& InFileHeader, FTakeFileContext& InContext, TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient, UE::CaptureManager::FTaskProgress::FTask& InTask)
{
	using namespace UE::CaptureManager;

	FTcpConnectionReader Reader(*InClient);

	FString FilePathToBeSaved = FPaths::Combine(InTakeStoragaPath, InFileHeader.FileName);
	IFileManager& FileManager = IFileManager::Get();

	TUniquePtr<FArchive> Writer(FileManager.CreateFileWriter(*FilePathToBeSaved, EFileWrite::FILEWRITE_None));
	if (!Writer)
	{
		FText Message =
			FText::Format(LOCTEXT("HandleFileDownload_FailedToCreateFile", "Failed to create the file: {0}"),
						  FText::FromString(FilePathToBeSaved));
		return MakeError(MoveTemp(Message));
	}

	FMD5 MD5Generator;

	uint64 BytesLeft = InFileHeader.Length;
	const uint64 ChunkSize = 64 * 1024; // 64Kb
	while (BytesLeft != 0)
	{
		const uint64 DataToRead = FMath::Min(BytesLeft, ChunkSize);
		FUploadResult<TArray<uint8>> DataResult = FUploadDataMessage::DeserializeData(DataToRead, Reader);

		if (DataResult.HasError())
		{
			Writer->Close();

			return MakeError(DataResult.StealError());
		}

		TArray<uint8> Data = DataResult.StealValue();

		MD5Generator.Update(Data.GetData(), Data.Num());

		Writer->Serialize(const_cast<uint8*>(Data.GetData()), Data.Num());
		BytesLeft -= Data.Num();

		InContext.Progress += static_cast<float>(Data.Num()) / InContext.TotalLength;

		InTask.Update(InContext.Progress);
	}

	Writer->Close();

	FUploadResult<TStaticArray<uint8, FUploadDataMessage::HashSize>> HashResult = FUploadDataMessage::DeserializeHash(Reader);
	if (HashResult.HasError())
	{
		return MakeError(HashResult.StealError());
	}

	TStaticArray<uint8, FUploadDataMessage::HashSize> Hash;
	MD5Generator.Final(Hash.GetData());

	TStaticArray<uint8, FUploadDataMessage::HashSize> ArrivedHash = HashResult.StealValue();

	if (FMemory::Memcmp(Hash.GetData(), ArrivedHash.GetData(), Hash.Num()) != 0)
	{
		static constexpr int64 HashMismatch = -10;
		return MakeError(LOCTEXT("HandleFileDownload_HashMismatch", "Hash mismatch detected"), HashMismatch);
	}

	return MakeValue();
}

void FLiveLinkHubImportWorker::DeleteDownloadedData(const FString& InTakeStoragePath)
{
	IFileManager& FileManager = IFileManager::Get();
	FileManager.DeleteDirectory(*InTakeStoragePath, false, true);
}

void FLiveLinkHubImportWorker::SpawnIngestTask(TSharedPtr<FEditorMessenger> InMessenger,
	const FString& InDataStorage,
	const FGuid& InCaptureSourceId,
	const FString& InCaptureSourceName,
	const FGuid& InTakeUploadId,
	TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress)
{
	using namespace UE::CaptureManager;

	AsyncTask(ENamedThreads::AnyThread, [
		SharedMessenger = MoveTemp(InMessenger),
		DataStorage = InDataStorage,
		CaptureSourceId = InCaptureSourceId,
		CaptureSourceName = InCaptureSourceName,
		TakeUploadId = InTakeUploadId,
		TaskProgress = MoveTemp(InTaskProgress)
	]()
		{
			TValueOrError<FIngestProcessResult, FText> ProcessResult =
				FIngestCaptureDataProcess::StartIngestProcess(DataStorage, CaptureSourceName, TakeUploadId);

			if (ProcessResult.HasValue())
			{
				FIngestProcessResult IngestProcessResult = ProcessResult.GetValue();

				ExecuteOnGameThread(TEXT("IngestAssetCreation"), [SharedMessenger, IngestProcessResult = MoveTemp(IngestProcessResult), CaptureSourceId, CaptureSourceName, TakeUploadId]()
					{
						FIngestAssetCreator::FPerTakeCallback Callback = FIngestAssetCreator::FPerTakeCallback(
							FIngestAssetCreator::FPerTakeCallback::Type::CreateLambda([SharedMessenger, TakeUploadId, CaptureSourceId](TPair<int32, FIngestAssetCreator::FAssetCreationResult> InResult)
								{
									if (InResult.Value.HasError())
									{
										FAssetCreationError Error = InResult.Value.StealError();

										PublishDoneEvent(SharedMessenger, CaptureSourceId, TakeUploadId, Error.GetMessage().ToString(), static_cast<int64>(Error.GetError()));
									}
								}), EDelegateExecutionThread::InternalThread);

						FIngestAssetCreator Creator;
						TArray<FCaptureDataAssetInfo> CaptureDataAssetInfos = Creator.CreateAssets_GameThread(IngestProcessResult.AssetsData, MoveTemp(Callback));
						if (CaptureDataAssetInfos.IsEmpty())
						{
							// Error handled in per take callback
							return;
						}

						const bool bAutoSave = GetDefault<UCaptureManagerEditorSettings>()->bAutoSaveAssets;
						UFootageCaptureData* CaptureData = CreateFootageCaptureDataAsset_GameThread(
							IngestProcessResult.TakeIngestPackagePath,
							MoveTemp(CaptureDataAssetInfos[0]),
							IngestProcessResult.CaptureDataTakeInfo,
							bAutoSave);

						if (CaptureData)
						{
							PublishDoneEvent(SharedMessenger, CaptureSourceId, TakeUploadId);
						}
						else
						{
							PublishDoneEvent(SharedMessenger, CaptureSourceId, TakeUploadId, TEXT("Failed to create CaptureData asset"), static_cast<int64>(EAssetCreationError::InternalError));
						}
					});
			}
			else
			{
				PublishDoneEvent(SharedMessenger, CaptureSourceId, TakeUploadId, TEXT("Failed to create CaptureData asset"), static_cast<int64>(EAssetCreationError::InternalError));
			}
		});
}

}

#undef LOCTEXT_NAMESPACE
