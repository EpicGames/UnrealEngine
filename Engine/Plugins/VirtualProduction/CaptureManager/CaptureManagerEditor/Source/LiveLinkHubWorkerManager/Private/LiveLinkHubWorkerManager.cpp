// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubWorkerManager.h"

#include "LiveLinkHubWorkerLog.h"

#include "LiveLinkHubExportServerModule.h"

DEFINE_LOG_CATEGORY(LogLiveLinkHubWorkerManager);

FLiveLinkHubWorkerManager::FLiveLinkHubWorkerManager()
{
	using namespace UE::CaptureManager::Private;

	Messenger = MakeShared<FLiveLinkHubImportWorker::FEditorMessenger>();

	Messenger->SetConnectionHandler(FConnectAcceptor::FConnectAccepted::CreateRaw(this, &FLiveLinkHubWorkerManager::ConnectAccepted),
		FConnectAcceptor::FConnectionLostHandler::CreateRaw(this, &FLiveLinkHubWorkerManager::ConnectionLost));
}

FLiveLinkHubWorkerManager::~FLiveLinkHubWorkerManager()
{
	FScopeLock Lock(&Mutex);
	Workers.Empty();
}

void FLiveLinkHubWorkerManager::Disconnect()
{
	Messenger->Disconnect();
}

bool FLiveLinkHubWorkerManager::IsConnected() const 
{
	return Messenger->IsConnected();
}

// Has to be sent from this Messenger because the FMessageBridge on LLH side requires a message to register the remote
// Endpoint in order for other messages to go through. In this case, that message has to be Discovery Response.
void FLiveLinkHubWorkerManager::SendDiscoveryResponse(FDiscoveryResponse* Response, FMessageAddress Receiver)
{
	Messenger->SendDiscoveryResponse(Response, Receiver);
}

TWeakPtr<UE::CaptureManager::Private::FLiveLinkHubImportWorker> FLiveLinkHubWorkerManager::AddWorker(FMessageAddress InServer)
{
	using namespace UE::CaptureManager::Private;

	FScopeLock Lock(&Mutex);

	TSharedPtr<FLiveLinkHubImportWorker>& AddedWorker = Workers.Add(InServer.ToString(), FLiveLinkHubImportWorker::Create(Messenger));

	return AddedWorker;
}

void FLiveLinkHubWorkerManager::RemoveWorker(FMessageAddress InServer)
{
	FScopeLock Lock(&Mutex);

	Workers.Remove(InServer.ToString());
}

FConnectResponse* FLiveLinkHubWorkerManager::ConnectAccepted(const FConnectRequest& InRequest, const FMessageAddress& InAddress)
{
	FLiveLinkHubExportServerModule& LiveLinkHubExportServerModule = 
		FModuleManager::LoadModuleChecked<FLiveLinkHubExportServerModule>("LiveLinkHubExportServer");

	FConnectResponse* Response = FMessageEndpoint::MakeMessage<FConnectResponse>();

	if (LiveLinkHubExportServerModule.IsExportServerRunning())
	{
		UE_LOGF(LogLiveLinkHubWorkerManager, Display, "Connection accepted from: %ls", *InAddress.ToString());

		Messenger->SetAddress(InAddress);

		AddWorker(InAddress);

		Response->Status = EStatus::Ok;
	}
	else
	{
		Response->Status = EStatus::InternalError;
		Response->Message = TEXT("Export server is offline.");
	}

	return Response;
}

void FLiveLinkHubWorkerManager::ConnectionLost(const FMessageAddress& InAddress)
{
	UE_LOGF(LogLiveLinkHubWorkerManager, Display, "Connection lost: %ls", *InAddress.ToString());

	RemoveWorker(InAddress);
}
