// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubWorkerManagerModule.h"

#include "LiveLinkHubExportServerModule.h"

#include "Misc/CoreDelegates.h"

void FLiveLinkHubWorkerManagerModule::StartupModule()
{
	Manager = MakeShared<FLiveLinkHubWorkerManager>();

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FLiveLinkHubWorkerManagerModule::PostEngineInit);

	static constexpr float DisconnectAfterDelaySeconds = 1.0f;
	FTSTicker::GetCoreTicker()
		.AddTicker(FTickerDelegate::CreateRaw(this, &FLiveLinkHubWorkerManagerModule::CheckExportServerAvailability), DisconnectAfterDelaySeconds);
}

void FLiveLinkHubWorkerManagerModule::ShutdownModule()
{
	Manager->Disconnect();
	Manager = nullptr;
}

TSharedRef<FLiveLinkHubWorkerManager> FLiveLinkHubWorkerManagerModule::GetManager()
{
	return Manager.ToSharedRef();
}

void FLiveLinkHubWorkerManagerModule::PostEngineInit()
{
}

bool FLiveLinkHubWorkerManagerModule::CheckExportServerAvailability(float InDelay)
{
	FLiveLinkHubExportServerModule& LiveLinkHubExportServerModule = 
		FModuleManager::LoadModuleChecked<FLiveLinkHubExportServerModule>("LiveLinkHubExportServer");

	if (!LiveLinkHubExportServerModule.IsExportServerRunning())
	{
		if (Manager && Manager->IsConnected())
		{
			Manager->Disconnect();
		}
	}

	return true;
}

IMPLEMENT_MODULE(FLiveLinkHubWorkerManagerModule, LiveLinkHubWorkerManager);
