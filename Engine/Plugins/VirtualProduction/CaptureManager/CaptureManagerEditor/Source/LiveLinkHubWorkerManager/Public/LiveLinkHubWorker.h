// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Messenger.h"
#include "Features/ConnectAcceptor.h"
#include "Features/UploadStateSender.h"

#include "MessageEndpoint.h"

#include "LiveLinkHubCaptureMessages.h"

#include "LiveLinkHubExportServer.h"
#include "UploadDataMessage.h"

#include "Async/StopToken.h"

#include "Async/TaskProgress.h"

namespace UE::CaptureManager::Private
{

class FLiveLinkHubImportWorker : public TSharedFromThis<FLiveLinkHubImportWorker>
{
private:

	struct FPrivateToken {};

public:
	using FEditorMessenger = FMessenger<FConnectAcceptor, FUploadStateSender>;

	static TSharedPtr<FLiveLinkHubImportWorker> Create(TWeakPtr<FEditorMessenger> InEditorMessenger);

	explicit FLiveLinkHubImportWorker(TWeakPtr<FEditorMessenger> InEditorMessenger, FPrivateToken);
	~FLiveLinkHubImportWorker();

private:

	struct FTakeFileContext
	{
		uint64 TotalLength = 0;
		uint64 RemainingLength = 0;
		float Progress = 0.f;
	};


	FString EvaluateSettings(const FUploadDataHeader& InHeader);
	bool HandleTakeDownload(FUploadDataHeader InHeader, TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient);
	FUploadVoidResult HandleFileDownload(const FString& InTakeStoragePath,
		const FUploadFileDataHeader& InFileHeader,
		FTakeFileContext& InContext,
		TSharedPtr<UE::CaptureManager::FTcpClientHandler> InClient,
		UE::CaptureManager::FTaskProgress::FTask& InTask);
	void DeleteDownloadedData(const FString& InTakeStoragePath);

	static void SpawnIngestTask(TSharedPtr<FEditorMessenger> InMessenger,
		const FString& InDataStorage,
		const FGuid& InCaptureSourceId,
		const FString& InCaptureSourceName,
		const FGuid& InTakeUploadId,
		TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress);

	TWeakPtr<FEditorMessenger> Messenger;
};

}