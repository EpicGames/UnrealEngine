// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubWorker.h"

#define UE_API LIVELINKHUBWORKERMANAGER_API


class FLiveLinkHubWorkerManager
{
public:

	UE_API FLiveLinkHubWorkerManager();
	UE_API ~FLiveLinkHubWorkerManager();

	UE_API void Disconnect();
	UE_API bool IsConnected() const;
	UE_API void SendDiscoveryResponse(FDiscoveryResponse* Response, FMessageAddress Receiver);

private:

	UE_API TWeakPtr<UE::CaptureManager::Private::FLiveLinkHubImportWorker> AddWorker(FMessageAddress InServer);

	UE_API void RemoveWorker(FMessageAddress InServer);

	UE_API FConnectResponse* ConnectAccepted(const FConnectRequest& InRequest, const FMessageAddress& InAddress);
	UE_API void ConnectionLost(const FMessageAddress& InAddress);

	TSharedPtr<UE::CaptureManager::Private::FLiveLinkHubImportWorker::FEditorMessenger> Messenger;

	FCriticalSection Mutex;
	TMap<FString, TSharedPtr<UE::CaptureManager::Private::FLiveLinkHubImportWorker>> Workers;
};

#undef UE_API
