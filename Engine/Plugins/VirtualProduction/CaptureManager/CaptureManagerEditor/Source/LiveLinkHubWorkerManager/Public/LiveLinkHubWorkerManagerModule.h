// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#include "LiveLinkHubWorkerManager.h"

#define UE_API LIVELINKHUBWORKERMANAGER_API

class FLiveLinkHubWorkerManagerModule : public IModuleInterface
{
public:

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	UE_API TSharedRef<FLiveLinkHubWorkerManager> GetManager();

private:

	UE_API void PostEngineInit();
	UE_API void EnginePreExit();

	UE_API bool CheckExportServerAvailability(float InDelay);

	TSharedPtr<FLiveLinkHubWorkerManager> Manager;
	FTSTicker::FDelegateHandle Delegate;
};

#undef UE_API
